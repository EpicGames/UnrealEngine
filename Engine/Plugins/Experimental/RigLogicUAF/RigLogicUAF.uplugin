{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "RigLogic for UAF",
	"Description": "RigLogic for UAF",
	"Category": "Animation",
	"CreatedBy": "Epic Games Inc.",
	"CreatedByURL": "",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"CanContainContent": true,
	"IsBetaVersion": false,
	"IsExperimentalVersion": true,
	"Installed": false,
	"Modules":
	[
		{
			"Name": "RigLogicUAF",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		},
		{
			"Name": "RigLogicUAFUncookedOnly",
			"Type": "UncookedOnly",
			"LoadingPhase": "Default"
		}
	],
	"Plugins":
	[
		{
			"Name": "RigLogic",
			"Enabled": true
		},
		{
			"Name": "UAF",
			"Enabled": true
		},
		{
			"Name": "UAFAnimGraph",
			"Enabled": true
		}
	]
}