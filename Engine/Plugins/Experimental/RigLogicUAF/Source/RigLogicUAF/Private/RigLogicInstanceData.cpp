// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigLogicInstanceData.h"
#include "RigLogicUAF.h"

#include "Animation/Skeleton.h"
#include "Engine/SkeletalMesh.h"
#include "DNA.h"
#include "DNAAsset.h"
#include "DNAAssetUserData.h"
#include "LODPose.h"
#include "SharedRigRuntimeContext.h"

namespace UE::UAF
{
	void FRigLogicInstanceData::Init(const UE::UAF::FReferencePose* ReferencePose)
	{
		LLM_SCOPE_BYNAME(TEXT("Animation/RigLogic"));

		if (!ReferencePose)
		{
			UE_LOGF(LogRigLogicUAF, Error, "Reference pose invalid.");
			return;
		}

		USkeletalMesh* SkeletalMesh = const_cast<USkeletalMesh*>(ReferencePose->SkeletalMesh.Get());
		if (!SkeletalMesh)
		{
			UE_LOGF(LogRigLogicUAF, Error, "No skeletal mesh assigned to reference pose.");
			return;
		}

		const USkeleton* Skeleton = SkeletalMesh->GetSkeleton();
		if (!Skeleton)
		{
			UE_LOGF(LogRigLogicUAF, Error, "No skeleton assigned to the skeletal mesh.");
			return;
		}

		// Resolve DNA: try the new UDNAAssetUserData first, then fall back to the legacy UDNAAsset
		TSharedPtr<FSharedRigRuntimeContext> SharedRigRuntimeContext;
		if (UDNAAssetUserData* DNAUserData = Cast<UDNAAssetUserData>(SkeletalMesh->GetAssetUserDataOfClass(UDNAAssetUserData::StaticClass())))
		{
			if (UDNA* DNA = DNAUserData->DNAAsset)
			{
				SharedRigRuntimeContext = DNA->GetRigRuntimeContext();
				CachedDNAIndexMapping = DNA->GetDNAIndexMapping(Skeleton, SkeletalMesh);
			}
		}
		else if (UDNAAsset* DNAAsset = Cast<UDNAAsset>(SkeletalMesh->GetAssetUserDataOfClass(UDNAAsset::StaticClass())))
		{
			SharedRigRuntimeContext = DNAAsset->GetRigRuntimeContext();
			CachedDNAIndexMapping = DNAAsset->GetDNAIndexMapping(Skeleton, SkeletalMesh);
		}

		if (!SharedRigRuntimeContext.IsValid())
		{
			UE_LOGF(LogRigLogicUAF, Warning, "No DNA asset assigned to the skeletal mesh.");
			return;
		}

		if (CachedRigRuntimeContext != SharedRigRuntimeContext)
		{
			CachedRigRuntimeContext = SharedRigRuntimeContext;
		}

		NumLODs = CachedRigRuntimeContext->VariableJointIndicesPerLOD.Num();

		InitBoneIndexMapping(ReferencePose);
		InitSparseAndDenseDriverJointMapping(ReferencePose);
	}

	void FRigLogicInstanceData::InitBoneIndexMapping(const UE::UAF::FReferencePose* ReferencePose)
	{
		LLM_SCOPE_BYNAME(TEXT("Animation/RigLogic"));

		RigLogicToSkeletonBoneIndexMappingPerLOD.Empty();
		RigLogicToSkeletonBoneIndexMappingPerLOD.SetNum(NumLODs);

		const TArrayView<const FBoneIndexType> MeshToPoseBoneIndexMap = ReferencePose->GetMeshBoneIndexToLODBoneIndexMap(); // mesh -> pose

		for (uint32 LODLevel = 0; LODLevel < NumLODs; ++LODLevel)
		{
			const TArray<uint16>& VariableJointIndices = CachedRigRuntimeContext->VariableJointIndicesPerLOD[LODLevel].Values;
			RigLogicToSkeletonBoneIndexMappingPerLOD[LODLevel].Reserve(VariableJointIndices.Num());

			for (const uint16 RigLogicJointIndex : VariableJointIndices)
			{
				// Get the mesh bone index from the RigLogic joint index. Index originating from a FindBoneName() on the mesh's reference skeleton.
				const FMeshPoseBoneIndex MeshBoneIndex = CachedDNAIndexMapping->JointsMapDNAIndicesToMeshPoseBoneIndices[RigLogicJointIndex];
				if (MeshBoneIndex.IsValid())
				{
					// Convert the skeleton bone index to a pose bone index for the given LOD level.
					const int32 PoseBoneIndex = MeshToPoseBoneIndexMap[MeshBoneIndex.GetInt()];
					if (ReferencePose->IsBoneEnabled(PoseBoneIndex, LODLevel))
					{
						RigLogicToSkeletonBoneIndexMappingPerLOD[LODLevel].Add({ RigLogicJointIndex, PoseBoneIndex });
					}
				}
				else
				{
					UE_LOGF(LogTemp, Warning, "Could not find bone in skeleton for RigLogic joint with index %i.", RigLogicJointIndex);
				}
			}
		}
	}

	void FRigLogicInstanceData::InitSparseAndDenseDriverJointMapping(const UE::UAF::FReferencePose* ReferencePose)
	{
		LLM_SCOPE_BYNAME(TEXT("Animation/RigLogic"));

		// Populate driver joint to raw control attribute mapping (used to feed RigLogic with inputs from the joint hierarchy)
		SparseDriverJointsToControlAttributesMapPerLOD.Empty();
		DenseDriverJointsToControlAttributesMapPerLOD.Empty();

		SparseDriverJointsToControlAttributesMapPerLOD.SetNum(NumLODs);
		DenseDriverJointsToControlAttributesMapPerLOD.SetNum(NumLODs);

		const TArrayView<const FBoneIndexType> MeshToPoseBoneIndexMap = ReferencePose->GetMeshBoneIndexToLODBoneIndexMap(); // mesh -> pose
		
		for (uint32 LODLevel = 0; LODLevel < NumLODs; ++LODLevel)
		{
			// Sparse mapping will likely remain empty so no reservation happens
			for (const auto& Mapping : CachedDNAIndexMapping->DriverJointsToControlAttributesMap)
			{
				const FMeshPoseBoneIndex MeshBoneIndex = Mapping.MeshPoseBoneIndex;
				if (MeshBoneIndex.IsValid())
				{
					// Convert the mesh bone index to a pose bone index for the given LOD level.
					const int32 PoseBoneIndex = MeshToPoseBoneIndexMap[MeshBoneIndex.GetInt()];
					if (ReferencePose->IsBoneEnabled(PoseBoneIndex, LODLevel))
					{
						if ((Mapping.RotationX != INDEX_NONE) && (Mapping.RotationY != INDEX_NONE) && (Mapping.RotationZ != INDEX_NONE) && (Mapping.RotationW != INDEX_NONE))
						{
							DenseDriverJointsToControlAttributesMapPerLOD[LODLevel].Add({ PoseBoneIndex, Mapping.DNAJointIndex, Mapping.RotationX, Mapping.RotationY, Mapping.RotationZ, Mapping.RotationW });
						}
						else
						{
							SparseDriverJointsToControlAttributesMapPerLOD[LODLevel].Add({ PoseBoneIndex, Mapping.DNAJointIndex, Mapping.RotationX, Mapping.RotationY, Mapping.RotationZ, Mapping.RotationW });
						}
					}
				}
			}
		}
	}
} // namespace UE::UAF
