// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigLogicInstanceDataPool.h"
#include "RigLogicUAF.h"

namespace UE::UAF
{
	TSharedPtr<FRigLogicInstanceData> FRigLogicInstanceDataPool::RequestData(const UE::UAF::FReferencePose* InReferencePose)
	{
		LLM_SCOPE_BYNAME(TEXT("Animation/RigLogic"));

		// Only lock while searching
		{
			FScopeLock Lock(&PoolAccessCriticalSection);

			if (TArray<TSharedPtr<FRigLogicInstanceData>>* Found = Datas.Find(InReferencePose->SkeletalMesh))
			{
				if (!Found->IsEmpty())
				{
					// We have pre-allocated data already, use one that is already there.
					return Found->Pop();
				}
			}
		}
		// We don't need to keep things locked here as we're creating a new object
		// Initializing a new data is not a lightweight operation, so be sure to close the lock first.

		// We don't have any pool datas in the array yet. Create a new one.
		TSharedPtr<FRigLogicInstanceData> NewObject = TSharedPtr<FRigLogicInstanceData>(new FRigLogicInstanceData());
		NewObject->Init(InReferencePose);
		return NewObject;
	}

	void FRigLogicInstanceDataPool::FreeData(TWeakObjectPtr<const USkeletalMesh> SkeletalMesh, TSharedPtr<FRigLogicInstanceData> InData)
	{
		LLM_SCOPE_BYNAME(TEXT("Animation/RigLogic"));
		FScopeLock Lock(&PoolAccessCriticalSection);

		// Do we have an element for the given skel mesh?
		TArray<TSharedPtr<FRigLogicInstanceData>>* Found = Datas.Find(SkeletalMesh);
		if (!Found)
		{
			// We don't. Create a new one and add the instance data to it.
			TArray<TSharedPtr<FRigLogicInstanceData>> NewArray;
			NewArray.Add(InData);
			Datas.Add(SkeletalMesh, NewArray);
		}
		else
		{
			// Found one, just add the instance data back in.
			Found->Add(InData);
		}
	}

	void FRigLogicInstanceDataPool::GarbageCollect()
	{
		LLM_SCOPE_BYNAME(TEXT("Animation/RigLogic"));
		FScopeLock Lock(&PoolAccessCriticalSection);

		for (auto Iterator = Datas.CreateIterator(); Iterator; ++Iterator)
		{
			// Remove cached instance datas for skeletal meshes that are not loaded anymore.
			if (!Iterator->Key.IsValid())
			{
				Iterator.RemoveCurrent();
			}
		}
	}

	void FRigLogicInstanceDataPool::Log()
	{
		FScopeLock Lock(&PoolAccessCriticalSection);

		UE_LOGF(LogRigLogicUAF, Display, "Pool data:");
		uint32 Counter = 0;
		for (auto Iterator = Datas.CreateIterator(); Iterator; ++Iterator)
		{
			if (Iterator->Key.IsValid())
			{
				const USkeletalMesh* SkeletalMesh = Iterator->Key.Get();
				UE_LOGF(LogRigLogicUAF, Display, "   - Skeletal Mesh %ls:", *SkeletalMesh->GetPathName());

				const TArray<TSharedPtr<FRigLogicInstanceData>>& InstanceDataArray = Iterator->Value;
				for (int32 i = 0; i < InstanceDataArray.Num(); ++i)
				{
					UE_LOGF(LogRigLogicUAF, Display, "      + InstanceData %i (0x%lld):", i, InstanceDataArray[i].Get());
				}
			}
			else
			{
				UE_LOGF(LogRigLogicUAF, Warning, "Entry %i linked to an invalid skeletal mesh.", Counter);
			}
			Counter++;
		}
	}
} // namespace UE::UAF