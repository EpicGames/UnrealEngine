// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigLogicTask.h"

#include "EvaluationVM/EvaluationVM.h"
#include "EvaluationVM/KeyframeState.h"
#include "TransformArrayOperations.h"

#include "RigLogicUAF.h"
#include "DNAIndexMapping.h"
#include "RigLogic.h"
#include "RigInstance.h"
#include "SharedRigRuntimeContext.h"
#include "RigLogicInstanceDataPool.h"

#include <tdm/TDM.h>

DEFINE_STAT(STAT_UAF_Task_RigLogic);

FUAFRigLogicTask FUAFRigLogicTask::Make(UE::UAF::FRigLogicTrait::FInstanceData* InstanceData)
{
	FUAFRigLogicTask Result;
	Result.TraitInstanceData = InstanceData;
	return Result;
}

void FUAFRigLogicTask::Execute(UE::UAF::FEvaluationVM& VM) const
{
	SCOPE_CYCLE_COUNTER(STAT_UAF_Task_RigLogic);

	if (VM.GetActiveNamedSet())
	{
		// TODO: Implement with new attribute runtime
		return;
	}

	// Pop pose, we'll re-use it for our result
	TUniquePtr<UE::UAF::FKeyframeState> Keyframe;
	if (!VM.PopValue(UE::UAF::KEYFRAME_STACK_NAME, Keyframe))
	{
		// We have no inputs, nothing to do.
		return;
	}

	// Lazy-init instance data.
	UE::UAF::FRigLogicModule& Module = FModuleManager::GetModuleChecked<UE::UAF::FRigLogicModule>("RigLogicUAF");
	TSharedPtr<UE::UAF::FRigLogicInstanceData> PoolInstanceData = Module.DataPool.RequestData(Keyframe->Pose.RefPose);

	if (const FSharedRigRuntimeContext* RigRuntimeContext = PoolInstanceData->CachedRigRuntimeContext.Get())
	{
		if (!TraitInstanceData->RigInstance)
		{
			TraitInstanceData->RigInstance = TUniquePtr<FRigInstance>(new FRigInstance(RigRuntimeContext->RigLogic.Get()));
		}
		FRigInstance* RigInstance = TraitInstanceData->RigInstance.Get();

		const FDNAIndexMapping* DNAIndexMapping = PoolInstanceData->CachedDNAIndexMapping.Get();
		if (RigInstance && RigRuntimeContext && DNAIndexMapping)
		{
			if (const FRigLogic* RigLogic = RigRuntimeContext->RigLogic.Get())
			{
				const int32 LODLevel = FMath::Clamp(Keyframe->Pose.LODLevel, 0, (int32)PoolInstanceData->NumLODs - 1);

				// 1. Feed input curves to RigLogic.
				FBlendedCurve& Curves = Keyframe->Curves;
				const TArrayView<const float> NeutralJointValues = PoolInstanceData->CachedRigRuntimeContext->RigLogic->GetNeutralJointValues();

				UpdateControlCurves(RigInstance,
					Curves,
					DNAIndexMapping->ControlAttributeCurves,
					DNAIndexMapping->MLMaskCurves,
					NeutralJointValues,
					PoolInstanceData->SparseDriverJointsToControlAttributesMapPerLOD[LODLevel],
					PoolInstanceData->DenseDriverJointsToControlAttributesMapPerLOD[LODLevel],
					Keyframe->Pose);

				// 2. Evaluate RigLogic.
				RigInstance->SetLOD(LODLevel);
				RigLogic->Calculate(RigInstance);

				// 3. Update UAF bone transforms and curves with RigLogic output.
				TArrayView<const uint16> VariableJointIndices = RigRuntimeContext->VariableJointIndicesPerLOD[LODLevel].Values;
				const TArrayView<const UE::UAF::FRigLogicBoneMapping> RigLogicIndexToMeshIndexMapping = PoolInstanceData->RigLogicToSkeletonBoneIndexMappingPerLOD[LODLevel];
				UpdateJoints(RigLogicIndexToMeshIndexMapping,
					VariableJointIndices,
					NeutralJointValues,
					RigInstance->GetJointOutputs(),
					Keyframe->Pose);

				UpdateBlendShapeCurves(DNAIndexMapping->MorphTargetCurvesPerLOD[LODLevel], RigInstance->GetBlendShapeOutputs(), Curves);
				UpdateAnimMapCurves(DNAIndexMapping->MaskMultiplierCurvesPerLOD[LODLevel], RigInstance->GetAnimatedMapOutputs(), Curves);
			}
		}
	}

	Module.DataPool.FreeData(Keyframe->Pose.RefPose->SkeletalMesh, PoolInstanceData);

	// Push our result back
	VM.PushValue(UE::UAF::KEYFRAME_STACK_NAME, MoveTemp(Keyframe));
}

void FUAFRigLogicTask::UpdateControlCurves(FRigInstance* InRigInstance,
	const FBlendedCurve& Curves,
	const FCachedIndexedCurve& ControlAttributeCurves,
	const FCachedIndexedCurve& MLMaskCurves,
	const TArrayView<const float> NeutralJointValues,
	const TArrayView<const UE::UAF::FPoseBoneControlAttributeMapping> SparseDriverJointsToControlAttributesMap,
	const TArrayView<const UE::UAF::FPoseBoneControlAttributeMapping> DenseDriverJointsToControlAttributesMap,
	const UE::UAF::FLODPoseStack& InputPose) const
{
	// Combine control attribute curve with input curve to get indexed curve to apply to rig
	// Curve elements that dont have a control mapping will have INDEX_NONE as their index
	UE::Anim::FNamedValueArrayUtils::Union(Curves, ControlAttributeCurves,
		[this, InRigInstance](const UE::Anim::FCurveElement& InCurveElement, const UE::Anim::FCurveElementIndexed& InControlAttributeCurveElement, UE::Anim::ENamedValueUnionFlags InFlags)
		{
			if (InControlAttributeCurveElement.Index != INDEX_NONE)
			{
				InRigInstance->SetRawControl(InControlAttributeCurveElement.Index, InCurveElement.Value);
			}
		});

	// Combine control attribute curve with input curve to get indexed curve to apply to rig
	// Curve elements that dont have a control mapping will have INDEX_NONE as their index
	UE::Anim::FNamedValueArrayUtils::Union(Curves, ControlAttributeCurves,
		[this, InRigInstance](const UE::Anim::FCurveElement& InCurveElement, const UE::Anim::FCurveElementIndexed& InControlAttributeCurveElement, UE::Anim::ENamedValueUnionFlags InFlags)
		{
			if (InControlAttributeCurveElement.Index != INDEX_NONE)
			{
				InRigInstance->SetRawControl(InControlAttributeCurveElement.Index, InCurveElement.Value);
			}
		});

	const uint16 LODLevel = InRigInstance->GetLOD();

	// The sparse mapping is NOT guaranteed to supply all quaternion attributes, so checks for each attribute mapping are present
	for (int32 MappingIndex = 0; MappingIndex < SparseDriverJointsToControlAttributesMap.Num(); ++MappingIndex)
	{
		const UE::UAF::FPoseBoneControlAttributeMapping& Mapping = SparseDriverJointsToControlAttributesMap[MappingIndex];
		const int32 PoseBoneIndex = Mapping.PoseBoneIndex;

		const UE::UAF::FTransformSoAAdapterConst TransformAdapter = InputPose.LocalTransforms[PoseBoneIndex];

		// Translation and Scale is currently not used here, so to avoid the overhead of checking them, they are simply ignored.
		// Should the need arise to use them as well, this code will need adjustment.
		const FQuat Rotation = TransformAdapter.GetRotation();
		const int32 AttrIndex = Mapping.DNAJointIndex * ATTR_COUNT_PER_JOINT;

		const tdm::fquat NeutralRotation{ NeutralJointValues[AttrIndex + 3], NeutralJointValues[AttrIndex + 4], NeutralJointValues[AttrIndex + 5], NeutralJointValues[AttrIndex + 6] };
		const tdm::fquat AbsPoseRotation{ static_cast<float>(Rotation.X), static_cast<float>(Rotation.Y), static_cast<float>(Rotation.Z), static_cast<float>(Rotation.W) };
		const tdm::fquat DeltaPoseRotation = tdm::inverse(NeutralRotation) * AbsPoseRotation;

		if (Mapping.RotationX != INDEX_NONE)
		{
			InRigInstance->SetRawControl(Mapping.RotationX, DeltaPoseRotation.x);
		}
		if (Mapping.RotationY != INDEX_NONE)
		{
			InRigInstance->SetRawControl(Mapping.RotationY, DeltaPoseRotation.y);
		}
		if (Mapping.RotationZ != INDEX_NONE)
		{
			InRigInstance->SetRawControl(Mapping.RotationZ, DeltaPoseRotation.z);
		}
		if (Mapping.RotationW != INDEX_NONE)
		{
			InRigInstance->SetRawControl(Mapping.RotationW, DeltaPoseRotation.w);
		}
	}

	// The dense mapping is guaranteed to supply all quaternion attributes, so NO checks for each attribute mapping are present
	for (int32 MappingIndex = 0; MappingIndex < DenseDriverJointsToControlAttributesMap.Num(); ++MappingIndex)
	{
		const UE::UAF::FPoseBoneControlAttributeMapping& Mapping = DenseDriverJointsToControlAttributesMap[MappingIndex];
		const int32 PoseBoneIndex = Mapping.PoseBoneIndex;

		const UE::UAF::FTransformSoAAdapterConst TransformAdapter = InputPose.LocalTransforms[PoseBoneIndex];

		// Translation and Scale is currently not used here, so to avoid the overhead of checking them, they are simply ignored.
		// Should the need arise to use them as well, this code will need adjustment.
		const FQuat Rotation = TransformAdapter.GetRotation();
		const int32 AttrIndex = Mapping.DNAJointIndex * ATTR_COUNT_PER_JOINT;

		const tdm::fquat NeutralRotation{ NeutralJointValues[AttrIndex + 3], NeutralJointValues[AttrIndex + 4], NeutralJointValues[AttrIndex + 5], NeutralJointValues[AttrIndex + 6] };
		const tdm::fquat AbsPoseRotation{ static_cast<float>(Rotation.X), static_cast<float>(Rotation.Y), static_cast<float>(Rotation.Z), static_cast<float>(Rotation.W) };
		const tdm::fquat DeltaPoseRotation = tdm::inverse(NeutralRotation) * AbsPoseRotation;

		InRigInstance->SetRawControl(Mapping.RotationX, DeltaPoseRotation.x);
		InRigInstance->SetRawControl(Mapping.RotationY, DeltaPoseRotation.y);
		InRigInstance->SetRawControl(Mapping.RotationZ, DeltaPoseRotation.z);
		InRigInstance->SetRawControl(Mapping.RotationW, DeltaPoseRotation.w);
	}

	if (MLMaskCurves.Num() != 0)
	{
		TArrayView<float> MaskBuffer = InRigInstance->GetMLMaskValues();
		UE::Anim::FNamedValueArrayUtils::Union(Curves, MLMaskCurves,
			[this, MaskBuffer](const UE::Anim::FCurveElement& InCurveElement, const UE::Anim::FCurveElementIndexed& InControlAttributeCurveElement, UE::Anim::ENamedValueUnionFlags InFlags)
			{
				if (InControlAttributeCurveElement.Index != INDEX_NONE)
				{
					MaskBuffer[InControlAttributeCurveElement.Index] = InCurveElement.Value;
				}
			});
	}
}

void FUAFRigLogicTask::UpdateJoints(const TArrayView<const UE::UAF::FRigLogicBoneMapping> RigLogicIndexToMeshIndexMapping,
	const TArrayView<const uint16> VariableJointIndices,
	const TArrayView<const float> NeutralJointValues,
	const TArrayView<const float> DeltaJointValues,
	UE::UAF::FLODPoseStack& OutputPose)
{
	const float* N = NeutralJointValues.GetData();
	const float* D = DeltaJointValues.GetData();

	for (const UE::UAF::FRigLogicBoneMapping& Mapping : RigLogicIndexToMeshIndexMapping)
	{
		const uint16 RigLogicJointIndex = Mapping.RigLogicJointIndex;
		const int32 PoseBoneIndex = Mapping.PoseBoneIndex;
		const uint16 AttrIndex = RigLogicJointIndex * ATTR_COUNT_PER_JOINT;

		UE::UAF::FTransformSoAAdapter TransformAdapter = OutputPose.LocalTransforms[PoseBoneIndex];

		TransformAdapter.SetTranslation(FVector((N[AttrIndex + 0] + D[AttrIndex + 0]), (N[AttrIndex + 1] + D[AttrIndex + 1]), (N[AttrIndex + 2] + D[AttrIndex + 2])));
		TransformAdapter.SetRotation(FQuat(N[AttrIndex + 3], N[AttrIndex + 4], N[AttrIndex + 5], N[AttrIndex + 6]) * FQuat(D[AttrIndex + 3], D[AttrIndex + 4], D[AttrIndex + 5], D[AttrIndex + 6]));
		TransformAdapter.SetScale3D(FVector((N[AttrIndex + 7] + D[AttrIndex + 7]), (N[AttrIndex + 8] + D[AttrIndex + 8]), (N[AttrIndex + 9] + D[AttrIndex + 9])));
	}
}

void FUAFRigLogicTask::UpdateBlendShapeCurves(const FCachedIndexedCurve& MorphTargetCurves, const TArrayView<const float> BlendShapeValues, FBlendedCurve& OutputCurves)
{
	UE::Anim::FNamedValueArrayUtils::Union(OutputCurves, MorphTargetCurves, [&BlendShapeValues](UE::Anim::FCurveElement& InOutResult, const UE::Anim::FCurveElementIndexed& InSource, UE::Anim::ENamedValueUnionFlags InFlags)
		{
			if (BlendShapeValues.IsValidIndex(InSource.Index))
			{
				InOutResult.Value = BlendShapeValues[InSource.Index];
				InOutResult.Flags |= UE::Anim::ECurveElementFlags::MorphTarget;
			}
		});
}

void FUAFRigLogicTask::UpdateAnimMapCurves(const FCachedIndexedCurve& MaskMultiplierCurves, const TArrayView<const float> AnimMapOutputs, FBlendedCurve& OutputCurves)
{
	UE::Anim::FNamedValueArrayUtils::Union(OutputCurves, MaskMultiplierCurves, [&AnimMapOutputs](UE::Anim::FCurveElement& InOutResult, const UE::Anim::FCurveElementIndexed& InSource, UE::Anim::ENamedValueUnionFlags InFlags)
		{
			if (AnimMapOutputs.IsValidIndex(InSource.Index))
			{
				InOutResult.Value = AnimMapOutputs[InSource.Index];
				InOutResult.Flags |= UE::Anim::ECurveElementFlags::Material;
			}
		});
}