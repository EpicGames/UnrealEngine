// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigLogicTrait.h"

#include "RigLogicUAF.h"
#include "RigLogicTask.h"
#include "TraitCore/ExecutionContext.h"

namespace UE::UAF
{
	AUTO_REGISTER_ANIM_TRAIT(FRigLogicTrait)

		// Trait implementation boilerplate
#define TRAIT_INTERFACE_ENUMERATOR(GeneratorMacro) \
		GeneratorMacro(IEvaluate) \
		GeneratorMacro(IHierarchy) \
		GeneratorMacro(IUpdate) \
		GeneratorMacro(IUpdateTraversal) \

		GENERATE_ANIM_TRAIT_IMPLEMENTATION(FRigLogicTrait, TRAIT_INTERFACE_ENUMERATOR, NULL_ANIM_TRAIT_INTERFACE_ENUMERATOR, NULL_ANIM_TRAIT_EVENT_ENUMERATOR)
#undef TRAIT_INTERFACE_ENUMERATOR

	void FRigLogicTrait::OnBecomeRelevant(FUpdateTraversalContext& Context, const TTraitBinding<IUpdate>& Binding, const FTraitUpdateState& TraitState) const
	{
		LLM_SCOPE_BYNAME(TEXT("Animation/RigLogic"));

		const FSharedData* SharedData = Binding.GetSharedData<FSharedData>();
		FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();

		if (!InstanceData->Input.IsValid())
		{
			InstanceData->Input = Context.AllocateNodeInstance(Binding, SharedData->Input);
		}

		FRigLogicModule& Module = FModuleManager::GetModuleChecked<FRigLogicModule>("RigLogicUAF");
		Module.DataPool.GarbageCollect();
	}

	void FRigLogicTrait::PostEvaluate(FEvaluateTraversalContext& Context, const TTraitBinding<IEvaluate>& Binding) const
	{
		FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();

		if (InstanceData->Input.IsValid())
		{
			// TODO: Only add the task in case we want to actually want to apply facial animation for the current LOD.
			//if (LODLevel < LODThreshold)
			{
				Context.AppendTask(FUAFRigLogicTask::Make(InstanceData));
			}
		}
	}

	void FRigLogicTrait::QueueChildrenForTraversal(FUpdateTraversalContext& Context, const TTraitBinding<IUpdateTraversal>& Binding, const FTraitUpdateState& TraitState, FUpdateTraversalQueue& TraversalQueue) const
	{
		const FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();

		if (InstanceData->Input.IsValid())
		{
			TraversalQueue.Push(InstanceData->Input, TraitState);
		}
	}

	uint32 FRigLogicTrait::GetNumChildren(const FExecutionContext& Context, const TTraitBinding<IHierarchy>& Binding) const
	{
		return 1;
	}

	void FRigLogicTrait::GetChildren(const FExecutionContext& Context, const TTraitBinding<IHierarchy>& Binding, FChildrenArray& Children) const
	{
		const FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();
		Children.Add(InstanceData->Input);
	}
} // namespace UE::UAF