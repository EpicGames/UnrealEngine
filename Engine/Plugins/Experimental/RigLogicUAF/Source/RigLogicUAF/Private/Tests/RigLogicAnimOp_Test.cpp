// Copyright Epic Games, Inc. All Rights Reserved.

#ifndef RIGLOGIC_UAF_TESTS_ENABLED
#define RIGLOGIC_UAF_TESTS_ENABLED 0
#endif

#if RIGLOGIC_UAF_TESTS_ENABLED

#if WITH_DEV_AUTOMATION_TESTS && WITH_EDITOR

#include "RigLogicAnimOp.h"
#include "Misc/AutomationTest.h"

#include "Animation/AnimData/AttributeIdentifier.h"
#include "Animation/BuiltInAttributeTypes.h"
#include "Animation/MorphTarget.h"
#include "Animation/Skeleton.h"
#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/Set.h"
#include "Engine/SkeletalMesh.h"
#include "Misc/MemStack.h"
#include "Misc/Paths.h"
#include "ReferenceSkeleton.h"
#include "UObject/Package.h"
#include "UObject/StrongObjectPtr.h"

#include "DNA.h"
#include "DNAAsset.h"
#include "DNAAssetUserData.h"
#include "DNAReader.h"
#include "DNAUtils.h"
#include "RigLogic.h"

#include "UAF/AbstractSkeleton/AbstractSkeletonSetBinding.h"
#include "UAF/AbstractSkeleton/AbstractSkeletonSetCollection.h"
#include "UAF/AnimOpCore/UAFAnimOpValueEvaluator.h"
#include "UAF/Attributes/AttributeNamedSet.h"
#include "UAF/Attributes/AttributeTypedSet.h"
#include "UAF/Attributes/EngineAttributes.h"
#include "UAF/ValueRuntime/AttributeMappingKey.h"
#include "UAF/ValueRuntime/PoseValueBundle.h"
#include "UAF/ValueRuntime/ValueSpace.h"

namespace UE::UAF::Tests
{
	static FString GetTestDNAFilePath()
	{
		const FString PluginDir = FPaths::Combine(FPaths::EnginePluginsDir(), TEXT("Animation"), TEXT("RigLogic"));
		return FPaths::Combine(PluginDir, TEXT("Content"), TEXT("Test"), TEXT("DNA"), TEXT("rl_unit_behavior_test.dna"));
	}

	// Build a transient skeleton whose joint hierarchy matches the joints reported by the
	// supplied DNA reader. Joint names are taken verbatim from the DNA so that
	// FDNAIndexMapping::MapJoints (RefSkeleton.FindBoneIndex by joint name) succeeds.
	// The DNA's self-parenting joint becomes the FReferenceSkeleton's INDEX_NONE root.
	static USkeleton* BuildSkeletonForDNA(const IDNAReader& DNAReader)
	{
		USkeleton* Skeleton = NewObject<USkeleton>(GetTransientPackage(), NAME_None, RF_Transient);
		check(Skeleton != nullptr);
		FReferenceSkeletonModifier Modifier(Skeleton);

		const uint16 JointCount = DNAReader.GetJointCount();
		for (uint16 JointIndex = 0; JointIndex < JointCount; ++JointIndex)
		{
			const FString JointName = DNAReader.GetJointName(JointIndex);
			const uint16 ParentIndex = DNAReader.GetJointParentIndex(JointIndex);

			int32 ParentBoneIndex = INDEX_NONE;
			if (ParentIndex != JointIndex)
			{
				const FString ParentName = DNAReader.GetJointName(ParentIndex);
				ParentBoneIndex = Modifier.FindBoneIndex(FName(*ParentName));
			}
			const FMeshBoneInfo Info(FName(*JointName), JointName, ParentBoneIndex);
			Modifier.Add(Info, FTransform::Identity);
		}
		return Skeleton;
	}

	// Build a transient SkeletalMesh that uses the supplied skeleton as its ref skeleton.
	// USkeletalMesh::RegisterMorphTarget ensureMsgf's on HasValidData() — true for cooked
	// morphs, but our synthetic UMorphTargets carry no morph LOD models. Sidestep that by
	// appending directly to the mesh's morph-target array, then call InitMorphTargets with
	// bInKeepEmptyMorphTargets=true to populate the index map.
	static USkeletalMesh* BuildSkeletalMeshForDNA(USkeleton* Skeleton, const IDNAReader& DNAReader)
	{
		USkeletalMesh* Mesh = NewObject<USkeletalMesh>(GetTransientPackage(), NAME_None, RF_Transient);
		check(Mesh != nullptr);
		Mesh->SetSkeleton(Skeleton);
		Mesh->SetRefSkeleton(Skeleton->GetReferenceSkeleton());

		const uint16 LODCount = DNAReader.GetLODCount();
		TSet<FName> RegisteredNames;
		TArray<UMorphTarget*> MorphTargetList = Mesh->GetMorphTargets();
		for (uint16 LODIndex = 0; LODIndex < LODCount; ++LODIndex)
		{
			TArrayView<const uint16> Mappings = DNAReader.GetMeshBlendShapeChannelMappingIndicesForLOD(LODIndex);
			for (const uint16 MappingIndex : Mappings)
			{
				const FMeshBlendShapeChannelMapping Mapping = DNAReader.GetMeshBlendShapeChannelMapping(MappingIndex);
				const FString MeshName = DNAReader.GetMeshName(Mapping.MeshIndex);
				const FString BlendShapeName = DNAReader.GetBlendShapeChannelName(Mapping.BlendShapeChannelIndex);
				const FName MorphTargetName(*(MeshName + TEXT("__") + BlendShapeName));

				if (!RegisteredNames.Contains(MorphTargetName))
				{
					UMorphTarget* MorphTarget = NewObject<UMorphTarget>(Mesh, MorphTargetName, RF_Transient);
					MorphTargetList.Add(MorphTarget);
					RegisteredNames.Add(MorphTargetName);
				}
			}
		}
		Mesh->SetMorphTargets(MorphTargetList);
		Mesh->InitMorphTargets(/*bInKeepEmptyMorphTargets=*/true);
		return Mesh;
	}

	static UDNA* AttachDNAToMesh(USkeletalMesh* Mesh, const FString& DNAFilePath)
	{
		// ReadDNAAssetFromFile returns a UDNA outered to Mesh; LoadDNAFromFile returns
		// only the IDNAReader (which we don't need here -- UDNA wraps its own reader).
		UDNA* DNA = ReadDNAAssetFromFile(DNAFilePath, Mesh);
		if (DNA)
		{
			UDNAAssetUserData* UserData = NewObject<UDNAAssetUserData>(Mesh, NAME_None, RF_Transient);
			UserData->DNAAsset = DNA;
			Mesh->AddAssetUserData(UserData);
		}
		return DNA;
	}

	static UDNAAsset* AttachLegacyDNAToMesh(USkeletalMesh* Mesh, const FString& DNAFilePath)
	{
		UDNAAsset* DNAAsset = NewObject<UDNAAsset>(Mesh, NAME_None, RF_Transient);
		if (DNAAsset && DNAAsset->Init(DNAFilePath))
		{
			Mesh->AddAssetUserData(Cast<UAssetUserData>(DNAAsset));
		}
		return DNAAsset;
	}

	// Build a UAbstractSkeletonSetBinding populated with every bone and every curve
	// attribute the test DNA expects. The binding is the input the AnimOp evaluator uses
	// to construct a named set with bone-name and curve-name lookups; if a name is not
	// present here, the AnimOp will silently skip it.
	//
	// Everything goes into the NAME_None set. We attach an empty UAbstractSkeletonSetCollection
	// because AddBoneToSet bails out if the binding has no SetCollection at all -- even when
	// adding to NAME_None.
	static UAbstractSkeletonSetBinding* BuildSetBinding(USkeleton* Skeleton, const IDNAReader& DNAReader)
	{
		UAbstractSkeletonSetCollection* SetCollection = NewObject<UAbstractSkeletonSetCollection>(GetTransientPackage(), NAME_None, RF_Transient);
		check(SetCollection != nullptr);
		UAbstractSkeletonSetBinding* Binding = NewObject<UAbstractSkeletonSetBinding>(GetTransientPackage(), NAME_None, RF_Transient);
		check(Binding != nullptr);
		Binding->SetSetCollection(SetCollection);
		if (!Binding->SetSkeleton(Skeleton))
		{
			return nullptr;
		}

		// Bone bindings: every bone in the ref skeleton goes into the everything set.
		const FReferenceSkeleton& RefSkeleton = Skeleton->GetReferenceSkeleton();
		const int32 NumBones = RefSkeleton.GetNum();
		for (int32 BoneIndex = 0; BoneIndex < NumBones; ++BoneIndex)
		{
			Binding->AddBoneToSet(RefSkeleton.GetBoneName(BoneIndex), NAME_None);
		}

		// Curve attribute bindings. RigLogicInstanceData::Init resolves curve names against
		// the float typed set; if a name is not there, the lookup returns INDEX_NONE and
		// that curve becomes a silent no-op. So every name the AnimOp will ever query has
		// to be present here.
		UScriptStruct* FloatType = FFloatAnimationAttribute::StaticStruct();
		auto AddFloatAttr = [Binding, FloatType](const FName Name)
		{
			Binding->AddAttributeToSet(FAnimationAttributeIdentifier(Name, INDEX_NONE, NAME_None, FloatType), NAME_None);
		};

		// Raw controls: DNA names use "<obj>.<attr>", FDNAIndexMapping rewrites them to
		// "<obj>_<attr>" before producing FCurveElement names. Match that here.
		const uint16 RawControlCount = DNAReader.GetRawControlCount();
		for (uint16 ControlIndex = 0; ControlIndex < RawControlCount; ++ControlIndex)
		{
			const FString ControlName = DNAReader.GetRawControlName(ControlIndex);
			FString ObjectName, AttrName;
			if (ControlName.Split(TEXT("."), &ObjectName, &AttrName))
			{
				AddFloatAttr(FName(*(ObjectName + TEXT("_") + AttrName)));
			}
			else
			{
				AddFloatAttr(FName(*ControlName));
			}
		}

		// Morph target curves: <mesh>__<blendshape>, every LOD.
		const uint16 LODCount = DNAReader.GetLODCount();
		TSet<FName> SeenMorphs;
		for (uint16 LODIndex = 0; LODIndex < LODCount; ++LODIndex)
		{
			const TArrayView<const uint16> MappingIndicesForLOD = DNAReader.GetMeshBlendShapeChannelMappingIndicesForLOD(LODIndex);
			for (const uint16 MappingIndex : MappingIndicesForLOD)
			{
				const FMeshBlendShapeChannelMapping Mapping = DNAReader.GetMeshBlendShapeChannelMapping(MappingIndex);
				const FString MeshName = DNAReader.GetMeshName(Mapping.MeshIndex);
				const FString ChannelName = DNAReader.GetBlendShapeChannelName(Mapping.BlendShapeChannelIndex);
				const FName MorphName(*(MeshName + TEXT("__") + ChannelName));
				bool bAlreadyIn = false;
				SeenMorphs.Add(MorphName, &bAlreadyIn);
				if (!bAlreadyIn)
				{
					AddFloatAttr(MorphName);
				}
			}
		}

		// Animated map curves: <obj>.<attr> -> <obj>_<attr>, every LOD.
		TSet<FName> SeenAnimMaps;
		for (uint16 LODIndex = 0; LODIndex < LODCount; ++LODIndex)
		{
			const TArrayView<const uint16> AnimMapIndicesForLOD = DNAReader.GetAnimatedMapIndicesForLOD(LODIndex);
			for (const uint16 AnimMapIndex : AnimMapIndicesForLOD)
			{
				const FString AnimMapName = DNAReader.GetAnimatedMapName(AnimMapIndex);
				FString ObjectName, AttrName;
				if (AnimMapName.Split(TEXT("."), &ObjectName, &AttrName))
				{
					const FName CurveName(*(ObjectName + TEXT("_") + AttrName));
					bool bAlreadyIn = false;
					SeenAnimMaps.Add(CurveName, &bAlreadyIn);
					if (!bAlreadyIn)
					{
						AddFloatAttr(CurveName);
					}
				}
			}
		}

		return Binding;
	}

	// Push a fresh pose bundle onto the evaluator stack with all bones at identity and
	// all curves zeroed. Mirrors the AnimNode test SeedSentinelPose in intent: stamp a known
	// starting state before each Evaluate so the test can assert on what RigLogic actually wrote.
	//
	// Identity is used here (not a sentinel) because the AnimOp has no upstream pose-link
	// writing a ref pose; non-variable joints simply retain whatever the input bundle held.
	// Using identity means non-variable joints stay at identity, matching the same expected-
	// table entries the AnimNode tests use.
	static void PushIdentityPose(FUAFAnimOpValueEvaluator& Evaluator)
	{
		FPoseValueBundleStack Bundle(Evaluator.GetActiveNamedSet());
		Bundle.InitWithValueSpace(FValueSpace(EValueSpaceType::Local));
		(void)Bundle.GetBoundValueMaps().FindOrAdd<FBoneTransformAnimationAttribute>(FAttributeMappingKey::MakeFromTo<FBoneTransformAnimationAttribute>());
		(void)Bundle.GetBoundValueMaps().FindOrAdd<FFloatAnimationAttribute>(FAttributeMappingKey::MakeFromTo<FFloatAnimationAttribute>());
		if (TBoundValueMap<FBoneTransformAnimationAttribute>* BoneTransforms = Bundle.FindBoneTransforms())
		{
			const int32 NumBones = BoneTransforms->Num();
			FBoneTransformAnimationAttribute* BoneData = BoneTransforms->GetData();
			for (int32 BoneIndex = 0; BoneIndex < NumBones; ++BoneIndex)
			{
				BoneData[BoneIndex].Value = FTransform::Identity;
			}
		}
		if (TBoundValueMap<FFloatAnimationAttribute>* FloatCurves = Bundle.FindFloatCurves())
		{
			const int32 NumCurves = FloatCurves->Num();
			FFloatAnimationAttribute* CurveData = FloatCurves->GetData();
			for (int32 CurveIndex = 0; CurveIndex < NumCurves; ++CurveIndex)
			{
				CurveData[CurveIndex].Value = 0.f;
			}
		}
		FPoseValueBundleCoWRef Ref = FPoseValueBundleCoWRef::MakeFrom(static_cast<FPoseValueBundle&&>(Bundle));
		Evaluator.GetEvaluationStack().Push(MoveTemp(Ref));
	}

	// Returns the top-of-stack bundle as a mutable reference. The bundle must already
	// exist on the stack (push first, then access).
	static FPoseValueBundle& TopBundle(FUAFAnimOpValueEvaluator& Evaluator)
	{
		FPoseValueBundleCoWRef* Ref = Evaluator.GetEvaluationStack().PeekMutable(0);
		check(Ref != nullptr);
		return *FPoseValueBundle::AsMutable(*Ref);
	}

	static TMap<FName, float> SnapshotCurves(const FPoseValueBundle& Bundle)
	{
		TMap<FName, float> Result;
		const TBoundValueMap<FFloatAnimationAttribute>* FloatCurves = Bundle.FindFloatCurves();
		if (FloatCurves == nullptr)
		{
			return Result;
		}

		const FAttributeTypedSetPtr TypedSet = FloatCurves->GetTypedSet();
		if (!TypedSet.IsValid())
		{
			return Result;
		}

		const int32 Num = FloatCurves->Num();
		const FFloatAnimationAttribute* CurveData = FloatCurves->GetData();
		for (int32 i = 0; i < Num; ++i)
		{
			const FName Name = TypedSet->GetName(FAttributeSetIndex(i));
			Result.Add(Name, CurveData[i].Value);
		}
		return Result;
	}

	static void SetRawControlCurve(FPoseValueBundle& Bundle, FName CurveName, float Value)
	{
		if (TBoundValueMap<FFloatAnimationAttribute>* FloatCurves = Bundle.FindFloatCurves())
		{
			const FAttributeTypedSetPtr TypedSet = FloatCurves->GetTypedSet();
			if (TypedSet.IsValid())
			{
				const FAttributeSetIndex Index = TypedSet->FindIndex(CurveName);
				if (Index.IsValid())
				{
					(*FloatCurves)[Index].Value = Value;
				}
			}
		}
	}

	// Holds every pinned UObject for a single test run. Test bodies construct one of
	// these, drive the AnimOp through it, then let it go out of scope.
	struct FAnimOpFixture
	{
		FMemMark MemMark;

		TStrongObjectPtr<USkeleton> Skeleton;
		TStrongObjectPtr<USkeletalMesh> Mesh;
		TStrongObjectPtr<UDNA> DNA;
		TStrongObjectPtr<UAbstractSkeletonSetCollection> SetCollection;
		TStrongObjectPtr<UAbstractSkeletonSetBinding> Binding;

		TSharedPtr<IDNAReader> DNAReader;
		TUniquePtr<FUAFAnimOpValueEvaluator> Evaluator;

		TArray<FAttributeSetIndex> DNAJointToSetIndex;
		TArray<FName> ExpectedMorphCurveNames;
		TArray<FName> ExpectedAnimMapCurveNames;
		TArray<FName> RawControlNames;

		FAnimOpFixture() : MemMark(FMemStack::Get())
		{
		}

		// Returns false if any step of the bootstrap fails; the caller should AddError and
		// return false from RunTest. Splitting construction from Setup keeps the destructor
		// safe even when the bootstrap fails partway through.
		bool Setup(FAutomationTestBase& Test, bool bUseLegacyDNA)
		{
			const FString DNAFilePath = GetTestDNAFilePath();
			if (!FPaths::FileExists(DNAFilePath))
			{
				Test.AddError(FString::Printf(TEXT("Test DNA not found at %s"), *DNAFilePath));
				return false;
			}

			DNAReader = LoadDNAFromFile(DNAFilePath);
			if (!DNAReader.IsValid())
			{
				Test.AddError(FString::Printf(TEXT("LoadDNAFromFile returned an invalid reader for %s"), *DNAFilePath));
				return false;
			}

			Skeleton.Reset(BuildSkeletonForDNA(*DNAReader));
			Mesh.Reset(BuildSkeletalMeshForDNA(Skeleton.Get(), *DNAReader));

			if (bUseLegacyDNA)
			{
				if (AttachLegacyDNAToMesh(Mesh.Get(), DNAFilePath) == nullptr)
				{
					Test.AddError(FString::Printf(TEXT("Failed to attach legacy UDNAAsset to mesh from %s"), *DNAFilePath));
					return false;
				}
			}
			else
			{
				DNA.Reset(AttachDNAToMesh(Mesh.Get(), DNAFilePath));
				if (!DNA.IsValid())
				{
					Test.AddError(FString::Printf(TEXT("Failed to attach UDNA to mesh from %s"), *DNAFilePath));
					return false;
				}
			}

			Binding.Reset(BuildSetBinding(Skeleton.Get(), *DNAReader));
			if (!Binding.IsValid())
			{
				Test.AddError(TEXT("Failed to build set binding"));
				return false;
			}
			SetCollection.Reset(const_cast<UAbstractSkeletonSetCollection*>(Binding->GetSetCollection().Get()));

			Evaluator = MakeUnique<FUAFAnimOpValueEvaluator>(Mesh.Get(), Binding.Get(), Mesh.Get(), NAME_None, 0);
			if (!Evaluator->GetActiveEvaluationContext().IsValid())
			{
				Test.AddError(TEXT("Evaluator context is invalid -- check binding/mesh/named-set wiring"));
				return false;
			}

			CacheDNAJointMapping();
			CacheExpectedCurveNames();
			CacheRawControlNames();
			return true;
		}

		void CacheDNAJointMapping()
		{
			DNAJointToSetIndex.Reset();
			const FAttributeNamedSetPtr& NamedSet = Evaluator->GetActiveNamedSet();
			if (!NamedSet.IsValid())
			{
				return;
			}

			const FAttributeTypedSetPtr BoneTypedSet = NamedSet->FindTypedSet(FBoneTransformAnimationAttribute::StaticStruct());
			if (!BoneTypedSet.IsValid())
			{
				return;
			}

			const uint16 JointCount = DNAReader->GetJointCount();
			DNAJointToSetIndex.Reserve(JointCount);
			for (uint16 JointIndex = 0; JointIndex < JointCount; ++JointIndex)
			{
				const FName JointName(*DNAReader->GetJointName(JointIndex));
				DNAJointToSetIndex.Add(BoneTypedSet->FindIndex(JointName));
			}
		}

		void CacheExpectedCurveNames()
		{
			const uint16 LODIndex = 0;
			ExpectedMorphCurveNames.Reset();
			TArrayView<const uint16> Mappings = DNAReader->GetMeshBlendShapeChannelMappingIndicesForLOD(LODIndex);
			for (const uint16 MappingIndex : Mappings)
			{
				const FMeshBlendShapeChannelMapping Mapping = DNAReader->GetMeshBlendShapeChannelMapping(MappingIndex);
				const FString MeshName = DNAReader->GetMeshName(Mapping.MeshIndex);
				const FString BlendShapeName = DNAReader->GetBlendShapeChannelName(Mapping.BlendShapeChannelIndex);
				ExpectedMorphCurveNames.Add(FName(*(MeshName + TEXT("__") + BlendShapeName)));
			}

			ExpectedAnimMapCurveNames.Reset();
			TArrayView<const uint16> AnimMapIndices = DNAReader->GetAnimatedMapIndicesForLOD(LODIndex);
			for (const uint16 AnimMapIndex : AnimMapIndices)
			{
				const FString AnimMapName = DNAReader->GetAnimatedMapName(AnimMapIndex);
				FString ObjectName, AttrName;
				if (AnimMapName.Split(TEXT("."), &ObjectName, &AttrName))
				{
					ExpectedAnimMapCurveNames.Add(FName(*(ObjectName + TEXT("_") + AttrName)));
				}
			}
		}

		void CacheRawControlNames()
		{
			const uint16 ControlCount = DNAReader->GetRawControlCount();
			RawControlNames.Reset(ControlCount);
			for (uint16 ControlIndex = 0; ControlIndex < ControlCount; ++ControlIndex)
			{
				const FString ControlName = DNAReader->GetRawControlName(ControlIndex);
				FString ObjectName, AttrName;
				if (ControlName.Split(TEXT("."), &ObjectName, &AttrName))
				{
					RawControlNames.Add(FName(*(ObjectName + TEXT("_") + AttrName)));
				}
				else
				{
					RawControlNames.Add(FName(*ControlName));
				}
			}
		}

		bool DidJointMove(const FPoseValueBundle& Bundle, uint16 DNAJointIndex, float Tolerance = KINDA_SMALL_NUMBER) const
		{
			if (!DNAJointToSetIndex.IsValidIndex(DNAJointIndex))
			{
				return false;
			}

			const FAttributeSetIndex SetIndex = DNAJointToSetIndex[DNAJointIndex];
			if (!SetIndex.IsValid())
			{
				return false;
			}

			const TBoundValueMap<FBoneTransformAnimationAttribute>* BoneTransforms = Bundle.FindBoneTransforms();
			if (BoneTransforms == nullptr)
			{
				return false;
			}

			const FTransform& T = (*BoneTransforms)[SetIndex].Value;
			return !T.Equals(FTransform::Identity, Tolerance);
		}

		TArray<uint16> CollectMovedJoints(const FPoseValueBundle& Bundle, float Tolerance = KINDA_SMALL_NUMBER) const
		{
			TArray<uint16> Moved;
			for (uint16 JointIndex = 0; JointIndex < static_cast<uint16>(DNAJointToSetIndex.Num()); ++JointIndex)
			{
				if (DidJointMove(Bundle, JointIndex, Tolerance))
				{
					Moved.Add(JointIndex);
				}
			}
			return Moved;
		}
	};

	// Seeds every raw control on the input pose to 0. Tests should call this before
	// any per-control overrides so the input curve set stays stable across Evaluate calls.
	// Setting all controls every frame matches the AnimNode test convention.
	static void SeedAllRawControlsZero(FPoseValueBundle& Bundle, const FAnimOpFixture& Fixture)
	{
		for (const FName& Name : Fixture.RawControlNames)
		{
			SetRawControlCurve(Bundle, Name, 0.0f);
		}
	}

	// Hand-captured fixture joint observations for the three input scenarios this suite targets.
	// Values were frozen by inspecting actual AnimOp output (and verified to match AnimNode output).
	struct FExpectedJoint
	{
		uint16 JointIndex;
		FVector Translation;
		FQuat Rotation;
		FVector Scale;
	};

	static void AssertJoint(FAutomationTestBase& Test, const FPoseValueBundle& Bundle, const FAnimOpFixture& Fixture, const FExpectedJoint& Expected, const TCHAR* ScenarioTag, float Tolerance = 1.e-3f)
	{
		if (!Fixture.DNAJointToSetIndex.IsValidIndex(Expected.JointIndex))
		{
			Test.AddError(FString::Printf(TEXT("%s J%u: out-of-range JointIndex"), ScenarioTag, Expected.JointIndex));
			return;
		}

		const FAttributeSetIndex SetIndex = Fixture.DNAJointToSetIndex[Expected.JointIndex];
		if (!SetIndex.IsValid())
		{
			Test.AddError(FString::Printf(TEXT("%s J%u: not in named set"), ScenarioTag, Expected.JointIndex));
			return;
		}

		const TBoundValueMap<FBoneTransformAnimationAttribute>* BoneTransforms = Bundle.FindBoneTransforms();
		if (BoneTransforms == nullptr)
		{
			Test.AddError(FString::Printf(TEXT("%s J%u: bundle has no bone transforms"), ScenarioTag, Expected.JointIndex));
			return;
		}

		const FTransform Posed = (*BoneTransforms)[SetIndex].Value;
		const FVector T = Posed.GetTranslation();
		const FQuat   R = Posed.GetRotation();
		const float QuatLength = static_cast<float>(R.Size());
		Test.TestEqual(*FString::Printf(TEXT("%s J%u rotation must be a unit quaternion"), ScenarioTag, Expected.JointIndex), QuatLength, 1.0f, 1.e-3f);
		const FVector S = Posed.GetScale3D();

		Test.TestEqual(*FString::Printf(TEXT("%s J%u TX"), ScenarioTag, Expected.JointIndex), static_cast<float>(T.X), static_cast<float>(Expected.Translation.X), Tolerance);
		Test.TestEqual(*FString::Printf(TEXT("%s J%u TY"), ScenarioTag, Expected.JointIndex), static_cast<float>(T.Y), static_cast<float>(Expected.Translation.Y), Tolerance);
		Test.TestEqual(*FString::Printf(TEXT("%s J%u TZ"), ScenarioTag, Expected.JointIndex), static_cast<float>(T.Z), static_cast<float>(Expected.Translation.Z), Tolerance);
		Test.TestEqual(*FString::Printf(TEXT("%s J%u RX"), ScenarioTag, Expected.JointIndex), static_cast<float>(R.X), static_cast<float>(Expected.Rotation.X), Tolerance);
		Test.TestEqual(*FString::Printf(TEXT("%s J%u RY"), ScenarioTag, Expected.JointIndex), static_cast<float>(R.Y), static_cast<float>(Expected.Rotation.Y), Tolerance);
		Test.TestEqual(*FString::Printf(TEXT("%s J%u RZ"), ScenarioTag, Expected.JointIndex), static_cast<float>(R.Z), static_cast<float>(Expected.Rotation.Z), Tolerance);
		Test.TestEqual(*FString::Printf(TEXT("%s J%u RW"), ScenarioTag, Expected.JointIndex), static_cast<float>(R.W), static_cast<float>(Expected.Rotation.W), Tolerance);
		Test.TestEqual(*FString::Printf(TEXT("%s J%u SX"), ScenarioTag, Expected.JointIndex), static_cast<float>(S.X), static_cast<float>(Expected.Scale.X), Tolerance);
		Test.TestEqual(*FString::Printf(TEXT("%s J%u SY"), ScenarioTag, Expected.JointIndex), static_cast<float>(S.Y), static_cast<float>(Expected.Scale.Y), Tolerance);
		Test.TestEqual(*FString::Printf(TEXT("%s J%u SZ"), ScenarioTag, Expected.JointIndex), static_cast<float>(S.Z), static_cast<float>(Expected.Scale.Z), Tolerance);
	}

}

// EditorContext + EngineFilter lets these run from the editor's Session Frontend -> Automation panel as well as from `RunTests RigLogic`.
namespace
{
	constexpr EAutomationTestFlags GTestFlags =	EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter;
}

// =============================================================================
// CacheBones - DNA resolution paths
// =============================================================================

namespace UE::UAF::Tests
{
	// Shared body for the two CacheBones tests below. Both DNA-attachment paths
	// (modern UDNAAssetUserData and legacy UDNAAsset) must produce identical
	// observable output: at least one DNA joint written, plus every morph curve
	// the DNA declares at LOD 0 present on the output pose.
	static void RunCacheBonesResolutionTest(FAutomationTestBase& Test, bool bUseLegacyDNA)
	{
		FAnimOpFixture Fixture;
		if (!Fixture.Setup(Test, bUseLegacyDNA))
		{
			return;
		}

		FRigLogicAnimOp Op;
		PushIdentityPose(*Fixture.Evaluator);
		FPoseValueBundle& Input = TopBundle(*Fixture.Evaluator);
		SeedAllRawControlsZero(Input, Fixture);
		Op.EvaluateValues(*Fixture.Evaluator);
		const FPoseValueBundle& Output = TopBundle(*Fixture.Evaluator);

		// At least one DNA joint should be written; if none are, InitFromPool never resolved
		// the runtime context and Evaluate early-returned.
		Test.TestFalse(TEXT("Some DNA joints should be written by Evaluate"), Fixture.CollectMovedJoints(Output).IsEmpty());

		// Every morph curve declared by the DNA at LOD0 should be present on the output.
		const TMap<FName, float> Curves = SnapshotCurves(Output);
		for (const FName& Expected : Fixture.ExpectedMorphCurveNames)
		{
			Test.TestTrue(*FString::Printf(TEXT("Morph curve missing: %s"), *Expected.ToString()), Curves.Contains(Expected));
		}
	}
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FRigLogicAnimOp_CacheBones_ResolvesDNAFromAssetUserData, "RigLogic.RigLogicUAF.CacheBones.ResolvesDNAFromAssetUserData", GTestFlags)
bool FRigLogicAnimOp_CacheBones_ResolvesDNAFromAssetUserData::RunTest(const FString&)
{
	UE::UAF::Tests::RunCacheBonesResolutionTest(*this, /*bUseLegacyDNA=*/false);
	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FRigLogicAnimOp_CacheBones_ResolvesDNAFromLegacyAssetUserData, "RigLogic.RigLogicUAF.CacheBones.ResolvesDNAFromLegacyAssetUserData", GTestFlags)
bool FRigLogicAnimOp_CacheBones_ResolvesDNAFromLegacyAssetUserData::RunTest(const FString&)
{
	UE::UAF::Tests::RunCacheBonesResolutionTest(*this, /*bUseLegacyDNA=*/true);
	return true;
}

// =============================================================================
// Evaluate - joint transforms written
// =============================================================================

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FRigLogicAnimOp_Evaluate_WritesNoNaN, "RigLogic.RigLogicUAF.Evaluate.WritesNoNaN", GTestFlags)
bool FRigLogicAnimOp_Evaluate_WritesNoNaN::RunTest(const FString&)
{
	using namespace UE::UAF::Tests;
	using namespace UE::UAF;

	FAnimOpFixture Fixture;
	if (!Fixture.Setup(*this, /*bUseLegacyDNA=*/false))
	{
		return false;
	}

	FRigLogicAnimOp Op;
	PushIdentityPose(*Fixture.Evaluator);
	FPoseValueBundle& Input = TopBundle(*Fixture.Evaluator);
	SeedAllRawControlsZero(Input, Fixture);
	for (const FName& ControlName : Fixture.RawControlNames)
	{
		SetRawControlCurve(Input, ControlName, 1.0e6f);
	}
	Op.EvaluateValues(*Fixture.Evaluator);
	const FPoseValueBundle& Output = TopBundle(*Fixture.Evaluator);

	const TBoundValueMap<FBoneTransformAnimationAttribute>* BoneTransforms = Output.FindBoneTransforms();
	if (BoneTransforms != nullptr)
	{
		const int32 Num = BoneTransforms->Num();
		const FBoneTransformAnimationAttribute* Data = BoneTransforms->GetData();
		const FAttributeTypedSetPtr TypedSet = BoneTransforms->GetTypedSet();
		for (int32 i = 0; i < Num; ++i)
		{
			if (Data[i].Value.ContainsNaN())
			{
				const FName BoneName = TypedSet.IsValid() ? TypedSet->GetName(FAttributeSetIndex(i)) : NAME_None;
				AddError(FString::Printf(TEXT("Bone transform contains NaN: %s"), *BoneName.ToString()));
				return false;
			}
		}
	}

	bool bCurveNaN = false;
	const TBoundValueMap<FFloatAnimationAttribute>* FloatCurves = Output.FindFloatCurves();
	if (FloatCurves != nullptr)
	{
		const int32 Num = FloatCurves->Num();
		const FFloatAnimationAttribute* Data = FloatCurves->GetData();
		for (int32 i = 0; i < Num; ++i)
		{
			if (FMath::IsNaN(Data[i].Value))
			{
				bCurveNaN = true;
				break;
			}
		}
	}
	TestFalse(TEXT("No output curve should be NaN"), bCurveNaN);
	return true;
}

// =============================================================================
// Fixtures values - captured from rl_unit_behavior_test.dna
//
// All numeric expectations below were captured by running RigLogic against the
// test DNA and recording the actual outputs. They capture RigLogic's behaviour
// at zero input, at a single non-zero input and at multiple non-zero inputs.
// All these test scenarios exercise specific conditional branches. If the DNA
// file changes, these fixture values must be re-captured and updated.
// =============================================================================

namespace UE::UAF::Tests::Fixtures
{
	// Scenario: all raw controls = 0. RigLogic writes each variable joint's (converted)
	// neutral pose. Translation matches the DNA's neutralJointTranslations after the
	// X/Y flip; rotation is the DNA's neutral Euler composed through the coord-system
	// pipeline; scale is identity (DNA has no neutralJointScales array).
	static const UE::UAF::Tests::FExpectedJoint ExpectedJointsAtZero[] =
	{
		{ 0, FVector(-1.0000000f, -1.0000000f, 1.0000000f), FQuat(-0.1675188f, -0.5709414f, 0.1675187f, 0.7860665f), FVector(1.0f, 1.0f, 1.0f) },
		{ 2, FVector(-3.0000000f, -3.0000000f, 3.0000000f), FQuat(0.0653920f, -0.0753745f, -0.0653920f, 0.9928575f), FVector(1.0f, 1.0f, 1.0f) },
		{ 4, FVector(-5.0000000f, -5.0000000f, 5.0000000f), FQuat(0.6710627f, 0.0971734f, -0.6710627f, 0.2998449f), FVector(1.0f, 1.0f, 1.0f) },
		{ 5, FVector(-6.0000000f, -6.0000000f, 6.0000000f), FQuat(0.1580251f, 0.1185940f, -0.1580252f, 0.9674665f), FVector(1.0f, 1.0f, 1.0f) },
		{ 6, FVector(-7.0000000f, -7.0000000f, 7.0000000f), FQuat(-0.1923898f, -0.4228496f, 0.1923897f, 0.8643902f), FVector(1.0f, 1.0f, 1.0f) },
		{ 7, FVector(-8.0000000f, -8.0000000f, 8.0000000f), FQuat(0.0510307f, -0.6977183f, -0.0510307f, 0.7127279f), FVector(1.0f, 1.0f, 1.0f) },
		{ 1, FVector::ZeroVector, FQuat::Identity, FVector::OneVector },
		{ 3, FVector::ZeroVector, FQuat::Identity, FVector::OneVector },
		{ 8, FVector::ZeroVector, FQuat::Identity, FVector::OneVector },
	};

	// At zero input, the animated-map conditional table contributes only those
	// entries whose (from <= 0 <= to) check passes. Walking the table for input=0:
	//   entry 0 (in 0 -> out 0): contributes 0*1 + 0   = 0
	//   entry 1 (in 1 -> out 1): contributes 0*0.9 + 0.5 = 0.5
	//   entry 6 (in 4 -> out 4): contributes 0*0.6 + 1   = 1
	//   entry 10 (in 6 -> out 6): contributes 0*0.6 + 0.4 = 0.4
	//   entry 14 (in 8 -> out 8): contributes 0*0.9 + 0.2 = 0.2
	// All other rows fail the from-bounds check and contribute nothing.
	// Animated-map names are A.A..A.J, mapped to curves A_A..A_J in the output.
	static const TPair<const TCHAR*, float> ExpectedAnimMapCurvesAtZero[] =
	{
		{ TEXT("A_A"), 0.0f },
		{ TEXT("A_B"), 0.5f },
		{ TEXT("A_C"), 0.0f },
		{ TEXT("A_D"), 0.0f },
		{ TEXT("A_E"), 1.0f },
		{ TEXT("A_F"), 0.0f },
		{ TEXT("A_G"), 0.4f },
		{ TEXT("A_H"), 0.0f },
		{ TEXT("A_I"), 0.2f },
		{ TEXT("A_J"), 0.0f },
	};
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FRigLogicAnimOp_Fixtures_AnimMapCurvesAtZero, "RigLogic.RigLogicUAF.Fixtures.AnimMapCurvesAtZero", GTestFlags)
bool FRigLogicAnimOp_Fixtures_AnimMapCurvesAtZero::RunTest(const FString&)
{
	using namespace UE::UAF::Tests;
	using namespace UE::UAF;

	FAnimOpFixture Fixture;
	if (!Fixture.Setup(*this, /*bUseLegacyDNA=*/false))
	{
		return false;
	}

	FRigLogicAnimOp Op;
	PushIdentityPose(*Fixture.Evaluator);
	FPoseValueBundle& Input = TopBundle(*Fixture.Evaluator);
	SeedAllRawControlsZero(Input, Fixture);
	// No raw controls set -> all inputs are zero.
	Op.EvaluateValues(*Fixture.Evaluator);
	const FPoseValueBundle& Output = TopBundle(*Fixture.Evaluator);

	const TMap<FName, float> Curves = SnapshotCurves(Output);
	for (const TPair<const TCHAR*, float>& Expected : Fixtures::ExpectedAnimMapCurvesAtZero)
	{
		const FName Name(Expected.Key);
		const float* ActualPtr = Curves.Find(Name);
		if (ActualPtr == nullptr)
		{
			AddError(FString::Printf(TEXT("Animated-map curve missing: %s"), Expected.Key));
			continue;
		}
		TestEqual(*FString::Printf(TEXT("Animated-map curve %s value"), Expected.Key), *ActualPtr, Expected.Value, 1.e-4f);
	}
	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FRigLogicAnimOp_Fixtures_BlendShapeCurvesAtZero, "RigLogic.RigLogicUAF.Fixtures.BlendShapeCurvesAtZero", GTestFlags)
bool FRigLogicAnimOp_Fixtures_BlendShapeCurvesAtZero::RunTest(const FString&)
{
	using namespace UE::UAF::Tests;
	using namespace UE::UAF;

	// At zero input, every blend-shape output is zero. Blend-shape outputs are
	// a direct selection from raw-control + PSD inputs (no constant offset),
	// so all-zero input -> all-zero output. Curve names in the output pose are
	// <mesh>__<blendshape> per FDNAIndexMapping::MapMorphTargets.
	FAnimOpFixture Fixture;
	if (!Fixture.Setup(*this, /*bUseLegacyDNA=*/false))
	{
		return false;
	}

	FRigLogicAnimOp Op;
	PushIdentityPose(*Fixture.Evaluator);
	FPoseValueBundle& Input = TopBundle(*Fixture.Evaluator);
	SeedAllRawControlsZero(Input, Fixture);
	Op.EvaluateValues(*Fixture.Evaluator);
	const FPoseValueBundle& Output = TopBundle(*Fixture.Evaluator);

	const TMap<FName, float> Curves = SnapshotCurves(Output);
	for (const FName& Name : Fixture.ExpectedMorphCurveNames)
	{
		const float* ActualPtr = Curves.Find(Name);
		if (ActualPtr == nullptr)
		{
			AddError(FString::Printf(TEXT("Morph curve missing: %s"), *Name.ToString()));
			continue;
		}
		TestEqual(*FString::Printf(TEXT("Morph curve %s should be zero at zero input"), *Name.ToString()), *ActualPtr, 0.0f, 1.e-4f);
	}
	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FRigLogicAnimOp_Fixtures_JointsAtZero, "RigLogic.RigLogicUAF.Fixtures.JointsAtZero", GTestFlags)
bool FRigLogicAnimOp_Fixtures_JointsAtZero::RunTest(const FString&)
{
	using namespace UE::UAF::Tests;
	using namespace UE::UAF;

	// At zero input, every joint group's delta is zero, so each variable joint's
	// final transform equals its DNA-declared neutral. The expected values are
	// captured in ExpectedJointsAtZero (translation after the X/Y coord flip,
	// rotation as the DNA's neutral Euler composed through RigLogic's pipeline,
	// and identity scale).
	//
	// Test DNA's joint groups touch joints {0, 2, 4, 5, 6, 7} at LOD 0; joints
	// {1, 3, 8} are not written by RigLogic and retain the input pose's identity
	// transforms (PushIdentityPose seeded them) -- ExpectedJointsAtZero pins both.
	FAnimOpFixture Fixture;
	if (!Fixture.Setup(*this, /*bUseLegacyDNA=*/false))
	{
		return false;
	}

	FRigLogicAnimOp Op;
	PushIdentityPose(*Fixture.Evaluator);
	FPoseValueBundle& Input = TopBundle(*Fixture.Evaluator);
	SeedAllRawControlsZero(Input, Fixture);
	Op.EvaluateValues(*Fixture.Evaluator);
	const FPoseValueBundle& Output = TopBundle(*Fixture.Evaluator);

	for (const FExpectedJoint& Expected : Fixtures::ExpectedJointsAtZero)
	{
		AssertJoint(*this, Output, Fixture, Expected, TEXT("Zero"));
	}
	return true;
}

// =============================================================================
// Fixtures values - non-zero input
//
// Drives raw control R.B (index 1) = 1.0 and asserts exact deltas relative to
// the neutral evaluation. All values derived from the DNA's JSON form. Any
// change to the test DNA invalidates these and they must be re-derived.
//
// Why R.B and not R.A?
//   PSDs are products of weighted raw inputs (PSD[row] = product over its
//   non-zero (row,col,value) of raw[col] * value). Single-control inputs other
//   than R.A drive PSDs to zero whenever any of their cols is zero, so for an
//   isolated R.B=1 input the entire post-PSD vector is just raw[1]=1 with all
//   PSDs zero. R.B happens to have a non-zero TRANSLATION coefficient in joint
//   group 0 (col 1, value 0.05 at output index 2 = TZ), giving a clean exact
//   translation delta to assert against. R.A's col-0 translation entry is 0,
//   which made it a poor choice.
//
// At R.B=1, post-PSD control = [0,1,0,0,0,0,0,0,0,0,0,0,...0] (PSD9..20 all zero).
//
// Joint group 0 (joint 0, 3 outputs x 7 inputs, row-major):
//   Inputs [0,1,2,3,6,7,8] -> col 1 active
//   M col 1 = [values[1], values[8], values[15]] = [0.05, 0.4, 0.75]
//   Outputs [2, 3, 5] -> joint 0 attrs 2 (TZ_DNA), 3 (RotEulX), 5 (RotEulZ)
//     joint 0 ATZ_DNA  += 0.05  -> UE Z += 0.05 (Z preserved)
//     joint 0 ARotEulX += 0.4
//     joint 0 ARotEulZ += 0.75
//
// Joint groups 1, 2, 3 do not include col 1 in their input lists, so they
// contribute nothing.
//
// Observable negations and swaps in the expected values are due to coordinate
// system conversion (DNA -> UE).
// =============================================================================

namespace UE::UAF::Tests::Fixtures
{
	// Scenario: R.B (raw control 1) = 1.0, all others 0. Only joint 0 gets a non-zero
	// delta (group 0 col 1: TZ_DNA +0.05, RotEulX +0.4 deg, RotEulZ +0.75 deg). All
	// other variable joints retain their neutral pose.
	static const UE::UAF::Tests::FExpectedJoint ExpectedJointsAtRB1[] =
	{
		{ 0, FVector(-1.0000000f, -1.0000000f, 1.0500000f), FQuat(-0.4908471f, -0.5487088f, 0.3415895f, 0.5842123f), FVector(1.0f, 1.0f, 1.0f) },
		{ 2, FVector(-3.0000000f, -3.0000000f, 3.0000000f), FQuat(0.0653920f, -0.0753745f, -0.0653920f, 0.9928575f), FVector(1.0f, 1.0f, 1.0f) },
		{ 4, FVector(-5.0000000f, -5.0000000f, 5.0000000f), FQuat(0.6710627f, 0.0971734f, -0.6710627f, 0.2998449f), FVector(1.0f, 1.0f, 1.0f) },
		{ 5, FVector(-6.0000000f, -6.0000000f, 6.0000000f), FQuat(0.1580251f, 0.1185940f, -0.1580252f, 0.9674665f), FVector(1.0f, 1.0f, 1.0f) },
		{ 6, FVector(-7.0000000f, -7.0000000f, 7.0000000f), FQuat(-0.1923898f, -0.4228496f, 0.1923897f, 0.8643902f), FVector(1.0f, 1.0f, 1.0f) },
		{ 7, FVector(-8.0000000f, -8.0000000f, 8.0000000f), FQuat(0.0510307f, -0.6977183f, -0.0510307f, 0.7127279f), FVector(1.0f, 1.0f, 1.0f) },
		{ 1, FVector::ZeroVector, FQuat::Identity, FVector::OneVector },
		{ 3, FVector::ZeroVector, FQuat::Identity, FVector::OneVector },
		{ 8, FVector::ZeroVector, FQuat::Identity, FVector::OneVector },
	};

	// Blend-shape outputs are direct passthrough: BS[outputIndices[k]] =
	// control[inputIndices[k]]. With control 1 = 1, only BS channel 1 is non-zero.
	// meshBlendShapeChannelMapping at LOD0 routes channel 1 to mesh 0 -> "MA__BB".
	static const TPair<const TCHAR*, float> ExpectedMorphCurves_RB1[] =
	{
		{ TEXT("MA__BA"), 0.0f },
		{ TEXT("MA__BB"), 1.0f },
		{ TEXT("MA__BC"), 0.0f },
		{ TEXT("MB__BD"), 0.0f },
		{ TEXT("MB__BE"), 0.0f },
		{ TEXT("MB__BF"), 0.0f },
		{ TEXT("MB__BG"), 0.0f },
		{ TEXT("MC__BH"), 0.0f },
		{ TEXT("MC__BI"), 0.0f },
	};

	static const TPair<const TCHAR*, float> ExpectedAnimMapCurves_RB1[] =
	{
		{ TEXT("A_A"), 0.0f },
		{ TEXT("A_B"), 1.0f },
		{ TEXT("A_C"), 0.0f },
		{ TEXT("A_D"), 0.0f },
		{ TEXT("A_E"), 1.0f },
		{ TEXT("A_F"), 0.0f },
		{ TEXT("A_G"), 0.4f },
		{ TEXT("A_H"), 0.0f },
		{ TEXT("A_I"), 0.2f },
		{ TEXT("A_J"), 0.0f },
	};
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FRigLogicAnimOp_Fixtures_SingleControlAtRB1, "RigLogic.RigLogicUAF.Fixtures.SingleControlAtRB1", GTestFlags)
bool FRigLogicAnimOp_Fixtures_SingleControlAtRB1::RunTest(const FString&)
{
	using namespace UE::UAF::Tests;
	using namespace UE::UAF;

	// Drives R.B=1.0 and asserts every observable at that input: per-joint
	// translation/rotation/scale plus the full set of morph-target and animated-map
	// curves the AnimOp emits. Single test instead of three so we don't pay the
	// fixture bootstrap cost more than once for the same input scenario.
	FAnimOpFixture Fixture;
	if (!Fixture.Setup(*this, /*bUseLegacyDNA=*/false))
	{
		return false;
	}

	FRigLogicAnimOp Op;
	PushIdentityPose(*Fixture.Evaluator);
	FPoseValueBundle& Input = TopBundle(*Fixture.Evaluator);
	SeedAllRawControlsZero(Input, Fixture);
	SetRawControlCurve(Input, Fixture.RawControlNames[1], 1.0f); // R.B = 1.0
	Op.EvaluateValues(*Fixture.Evaluator);
	const FPoseValueBundle& Output = TopBundle(*Fixture.Evaluator);

	for (const FExpectedJoint& Expected : Fixtures::ExpectedJointsAtRB1)
	{
		AssertJoint(*this, Output, Fixture, Expected, TEXT("RB1"));
	}

	const TMap<FName, float> Curves = SnapshotCurves(Output);
	for (const TPair<const TCHAR*, float>& Expected : Fixtures::ExpectedMorphCurves_RB1)
	{
		const FName Name(Expected.Key);
		const float* ActualPtr = Curves.Find(Name);
		if (ActualPtr == nullptr)
		{
			AddError(FString::Printf(TEXT("Morph curve missing: %s"), Expected.Key));
			continue;
		}
		TestEqual(*FString::Printf(TEXT("Morph curve %s value at R.B=1"), Expected.Key), *ActualPtr, Expected.Value, 1.e-4f);
	}
	for (const TPair<const TCHAR*, float>& Expected : Fixtures::ExpectedAnimMapCurves_RB1)
	{
		const FName Name(Expected.Key);
		const float* ActualPtr = Curves.Find(Name);
		if (ActualPtr == nullptr)
		{
			AddError(FString::Printf(TEXT("Animated-map curve missing: %s"), Expected.Key));
			continue;
		}
		TestEqual(*FString::Printf(TEXT("Animated-map curve %s value at R.B=1"), Expected.Key), *ActualPtr, Expected.Value, 1.e-4f);
	}

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FRigLogicAnimOp_Fixtures_RawControlScalesLinearly, "RigLogic.RigLogicUAF.Fixtures.RawControlScalesLinearly", GTestFlags)
bool FRigLogicAnimOp_Fixtures_RawControlScalesLinearly::RunTest(const FString&)
{
	using namespace UE::UAF::Tests;
	using namespace UE::UAF;

	// R.B=1 produces a non-zero translation delta on joint 0 (TZ_UE = 0.05).
	// At R.B=0.5 it must produce exactly half that delta. Verifies linearity
	// of the joint-group computation in the input control value.
	FAnimOpFixture Fixture;
	if (!Fixture.Setup(*this, /*bUseLegacyDNA=*/false))
	{
		return false;
	}

	FRigLogicAnimOp Op;

	PushIdentityPose(*Fixture.Evaluator);
	FPoseValueBundle& NeutralIn = TopBundle(*Fixture.Evaluator);
	SeedAllRawControlsZero(NeutralIn, Fixture);
	Op.EvaluateValues(*Fixture.Evaluator);
	const FTransform Neutral = (*TopBundle(*Fixture.Evaluator).FindBoneTransforms())[Fixture.DNAJointToSetIndex[0]].Value;

	PushIdentityPose(*Fixture.Evaluator);
	FPoseValueBundle& HalfIn = TopBundle(*Fixture.Evaluator);
	SeedAllRawControlsZero(HalfIn, Fixture);
	SetRawControlCurve(HalfIn, Fixture.RawControlNames[1], 0.5f);
	Op.EvaluateValues(*Fixture.Evaluator);
	const FTransform Half = (*TopBundle(*Fixture.Evaluator).FindBoneTransforms())[Fixture.DNAJointToSetIndex[0]].Value;

	PushIdentityPose(*Fixture.Evaluator);
	FPoseValueBundle& FullIn = TopBundle(*Fixture.Evaluator);
	SeedAllRawControlsZero(FullIn, Fixture);
	SetRawControlCurve(FullIn, Fixture.RawControlNames[1], 1.0f);
	Op.EvaluateValues(*Fixture.Evaluator);
	const FTransform Full = (*TopBundle(*Fixture.Evaluator).FindBoneTransforms())[Fixture.DNAJointToSetIndex[0]].Value;

	const FVector NeutralT = Neutral.GetTranslation();
	const FVector HalfDelta = Half.GetTranslation() - NeutralT;
	const FVector FullDelta = Full.GetTranslation() - NeutralT;

	TestEqual(TEXT("Joint 0 dTx scales linearly"), static_cast<float>(HalfDelta.X) * 2.0f, static_cast<float>(FullDelta.X), 1.e-3f);
	TestEqual(TEXT("Joint 0 dTy scales linearly"), static_cast<float>(HalfDelta.Y) * 2.0f, static_cast<float>(FullDelta.Y), 1.e-3f);
	TestEqual(TEXT("Joint 0 dTz scales linearly"), static_cast<float>(HalfDelta.Z) * 2.0f, static_cast<float>(FullDelta.Z), 1.e-3f);
	// Sanity: Full TZ delta must actually equal 0.05 (the fixture value).
	TestEqual(TEXT("Joint 0 dTz at R.B=1 equals expected value 0.05f"), static_cast<float>(FullDelta.Z), 0.05f, 1.e-3f);
	return true;
}

// =============================================================================
// Multi-control input - full coverage (joints, rotations, scales, all curves)
//
// At R.A=R.D=R.G=1, post-PSD vector has raw[0]=raw[3]=raw[6]=1 and PSD9=0.81;
// every other PSD evaluates to zero (each PSD is the product of weighted inputs,
// and any zero column kills the product).
//
// JOINT GROUPS active at LOD 0:
//   Group 0 (joint 0; 3 outputs x 7 inputs, inputs [0,1,2,3,6,7,8] -> [1,0,0,1,1,0,0]):
//     out 2  / J0 TZ_DNA = values[0,3,4]   = [0, 0.15, 0.20]   -> 0.35
//     out 3  / J0 RotEulX_DNA = values[7,10,11]  = [0.35, 0.50, 0.55] -> 1.40 deg
//     out 5  / J0 RotEulZ_DNA = values[14,17,18] = [0.70, 0.85, 0.90] -> 2.45 deg
//
//   Group 1 (joints 2, 4; 4 outputs x 5 inputs, inputs [3,4,7,8,9] -> [1,0,0,0,0.81]):
//     out 18 / J2 TX_DNA = 0.01 + 0.05*0.81 = 0.0505
//     out 20 / J2 TZ_DNA = 0.06 + 0.10*0.81 = 0.141
//     out 36 / J4 TX_DNA = 0.11 + 0.15*0.81 = 0.2315
//     out 38 / J4 TZ_DNA = 0.16 + 0.20*0.81 = 0.322
//
//   Group 2 (joints 6, 7; 3 outputs x 4 inputs, inputs [4,5,8,9] -> [0,0,0,0.81]):
//     out 55 / J6 TY_DNA = 0.47*0.81 = 0.3807
//     out 56 / J6 TZ_DNA = 0.69*0.81 = 0.5589
//     out 63 / J7 TX_DNA = 0.91*0.81 = 0.7371
//
//   Group 3 (joints 5, 7; 3 outputs x 4 inputs, inputs [2,5,6,8] -> [0,0,1,0]):
//     out 45 / J5 TX_DNA = 0.42
//     out 46 / J5 TY_DNA = 0.64
//     out 71 / J7 SZ_DNA = 0.86  (note: this is SCALE, attribute 8)
//
// COORDINATE-SYSTEM CONVERSION (DNA -> UE):
//   Translations: X and Y negated, Z preserved.
//   Rotations: composed via FQuat with the converted neutral; we don't pin
//   exact quat components and instead assert "rotation changed / didn't" plus
//   approximate angular distance.
//   Scales: NOT flipped. Final scale = neutralScale (1) + delta. Only J7's SZ
//   gets a delta (+0.86), so J7's final scale is (1, 1, 1.86).
//
// BLEND-SHAPE CHANNELS (direct passthrough at LOD 0):
//   inputIndices [0,1,2,3,6,7,8] -> outputs [0,1,2,3,6,7,8]
//   At raw[0]=raw[3]=raw[6]=1: BS[0]=1, BS[3]=1, BS[6]=1, all others 0
//   meshBlendShapeChannelMapping at LOD 0 routes BS channel k to mesh m via
//   the mapping table. Mappings (in order): mesh0:ch0, mesh0:ch1, mesh0:ch2,
//   mesh1:ch3, mesh1:ch4, mesh1:ch5, mesh1:ch6, mesh2:ch7, mesh2:ch8.
//   So output curves: MA__BA=1, MA__BB=0, MA__BC=0, MB__BD=1, MB__BE=0,
//   MB__BF=0, MB__BG=1, MC__BH=0, MC__BI=0.
//
// ANIMATED MAPS (15 conditional rows, all active at LOD 0):
//   At raw[0]=raw[3]=raw[6]=1, all other raw=0, walking the conditional table
//   for each row's (from <= raw[inputIdx] <= to) check, contributing
//   slope*raw[inputIdx] + cut to the output:
//     A_A: row 0 fires (1*1+0)             = 1.0
//     A_B: row 1 fires (0.9*0+0.5)         = 0.5
//     A_C: no row fires                    = 0
//     A_D: row 5 fires (0.7*1+0.3)         = 1.0  (raw[3]=1 in [0.7, 1])
//     A_E: row 6 fires (0.6*0+1)           = 1.0
//     A_F: no row fires                    = 0
//     A_G: row 10 fires (0.6*1+0.4)        = 1.0  (raw[6]=1 in [0, 1])
//     A_H: no row fires                    = 0
//     A_I: row 14 fires (0.9*0+0.2)        = 0.2
//     A_J: no row references output 9      = 0
// =============================================================================

namespace UE::UAF::Tests::Fixtures
{
	// Scenario: R.A=R.D=R.G=1.0. Activates PSD9 (=0.81) feeding groups 1 and 2; group 0
	// col 0/3/4 active for joint 0; group 3 col 2 active for joints 5 and 7. See the
	// MultiControlPSDActivation test header for the per-group derivation.
	static const UE::UAF::Tests::FExpectedJoint ExpectedJointsAtRABDG1[] =
	{
		{ 0, FVector(-1.0000000f, -1.0000000f, 1.3500000f), FQuat(-0.5243699f, -0.5404704f, 0.5859184f, -0.2993780f), FVector(1.0f, 1.0f, 1.0f) },
		{ 2, FVector(-3.0504999f, -3.0000000f, 3.1410000f), FQuat(0.0653920f, -0.0753745f, -0.0653920f, 0.9928575f), FVector(1.0f, 1.0f, 1.0f) },
		{ 4, FVector(-5.2315001f, -5.0000000f, 5.3220000f), FQuat(0.6710627f, 0.0971734f, -0.6710627f, 0.2998449f), FVector(1.0f, 1.0f, 1.0f) },
		{ 5, FVector(-6.4200001f, -6.6399999f, 6.0000000f), FQuat(0.1580251f, 0.1185940f, -0.1580252f, 0.9674665f), FVector(1.0f, 1.0f, 1.0f) },
		{ 6, FVector(-7.0000000f, -7.3807001f, 7.5588999f), FQuat(-0.1923898f, -0.4228496f, 0.1923897f, 0.8643902f), FVector(1.0f, 1.0f, 1.0f) },
		{ 7, FVector(-8.7370996f, -8.0000000f, 8.0000000f), FQuat(0.0510307f, -0.6977183f, -0.0510307f, 0.7127279f), FVector(1.0f, 1.0f, 1.8600000f) },
		{ 1, FVector::ZeroVector, FQuat::Identity, FVector::OneVector },
		{ 3, FVector::ZeroVector, FQuat::Identity, FVector::OneVector },
		{ 8, FVector::ZeroVector, FQuat::Identity, FVector::OneVector },
	};

	static const TPair<const TCHAR*, float> ExpectedMorphCurves_RABDG1[] =
	{
		{ TEXT("MA__BA"), 1.0f },
		{ TEXT("MA__BB"), 0.0f },
		{ TEXT("MA__BC"), 0.0f },
		{ TEXT("MB__BD"), 1.0f },
		{ TEXT("MB__BE"), 0.0f },
		{ TEXT("MB__BF"), 0.0f },
		{ TEXT("MB__BG"), 1.0f },
		{ TEXT("MC__BH"), 0.0f },
		{ TEXT("MC__BI"), 0.0f },
	};

	static const TPair<const TCHAR*, float> ExpectedAnimMapCurves_RABDG1[] =
	{
		{ TEXT("A_A"), 1.0f },
		{ TEXT("A_B"), 0.5f },
		{ TEXT("A_C"), 0.0f },
		{ TEXT("A_D"), 1.0f },
		{ TEXT("A_E"), 1.0f },
		{ TEXT("A_F"), 0.0f },
		{ TEXT("A_G"), 1.0f },
		{ TEXT("A_H"), 0.0f },
		{ TEXT("A_I"), 0.2f },
		{ TEXT("A_J"), 0.0f },
	};
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FRigLogicAnimOp_Fixtures_MultiControlPSDActivation, "RigLogic.RigLogicUAF.Fixtures.MultiControlPSDActivation", GTestFlags)
bool FRigLogicAnimOp_Fixtures_MultiControlPSDActivation::RunTest(const FString&)
{
	using namespace UE::UAF::Tests;
	using namespace UE::UAF;

	// --- Expected curves at R.A=R.D=R.G=1 ---
	FAnimOpFixture Fixture;
	if (!Fixture.Setup(*this, /*bUseLegacyDNA=*/false))
	{
		return false;
	}

	FRigLogicAnimOp Op;
	PushIdentityPose(*Fixture.Evaluator);
	FPoseValueBundle& Input = TopBundle(*Fixture.Evaluator);
	SeedAllRawControlsZero(Input, Fixture);
	SetRawControlCurve(Input, Fixture.RawControlNames[0], 1.0f); // R.A
	SetRawControlCurve(Input, Fixture.RawControlNames[3], 1.0f); // R.D
	SetRawControlCurve(Input, Fixture.RawControlNames[6], 1.0f); // R.G
	Op.EvaluateValues(*Fixture.Evaluator);
	const FPoseValueBundle& Output = TopBundle(*Fixture.Evaluator);

	// --- Joint observations: translation, rotation, scale (fixture table) ---
	for (const FExpectedJoint& Expected : Fixtures::ExpectedJointsAtRABDG1)
	{
		AssertJoint(*this, Output, Fixture, Expected, TEXT("RABDG1"));
	}

	// --- Curve observations: morph + animated map, every name ---
	const TMap<FName, float> Curves = SnapshotCurves(Output);
	for (const TPair<const TCHAR*, float>& E : Fixtures::ExpectedMorphCurves_RABDG1)
	{
		const float* Actual = Curves.Find(FName(E.Key));
		if (Actual == nullptr)
		{
			AddError(FString::Printf(TEXT("Morph curve missing: %s"), E.Key));
			continue;
		}
		TestEqual(*FString::Printf(TEXT("Morph curve %s"), E.Key), *Actual, E.Value, 1.e-4f);
	}
	for (const TPair<const TCHAR*, float>& E : Fixtures::ExpectedAnimMapCurves_RABDG1)
	{
		const float* Actual = Curves.Find(FName(E.Key));
		if (Actual == nullptr)
		{
			AddError(FString::Printf(TEXT("Animated-map curve missing: %s"), E.Key));
			continue;
		}
		TestEqual(*FString::Printf(TEXT("Animated-map curve %s"), E.Key), *Actual, E.Value, 1.e-4f);
	}

	return true;
}

// =============================================================================
// Conditional in active region (animated map)
//
// At zero input the animated-map A_B curve evaluates to 0.5 (entry 1: from=0,
// to=0.6 includes 0; output += 0.9*0 + 0.5 = 0.5).
// At R.B=0.5 entry 1 still fires (0.5 is in [0, 0.6]) and now contributes
//   slope*input + cut = 0.9*0.5 + 0.5 = 0.95
// Entry 2 (from=0.6, to=1) still fails because 0.5 < 0.6.
// So A_B at R.B=0.5 must be exactly 0.95. Verifies the conditional is firing
// in its linear region (slope*x + cut), not just at the boundary.
// =============================================================================

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FRigLogicAnimOp_Fixtures_ConditionalAtNonZeroInput, "RigLogic.RigLogicUAF.Fixtures.ConditionalAtNonZeroInput", GTestFlags)
bool FRigLogicAnimOp_Fixtures_ConditionalAtNonZeroInput::RunTest(const FString&)
{
	using namespace UE::UAF::Tests;
	using namespace UE::UAF;

	FAnimOpFixture Fixture;
	if (!Fixture.Setup(*this, /*bUseLegacyDNA=*/false))
	{
		return false;
	}

	FRigLogicAnimOp Op;
	PushIdentityPose(*Fixture.Evaluator);
	FPoseValueBundle& Input = TopBundle(*Fixture.Evaluator);
	SeedAllRawControlsZero(Input, Fixture);
	SetRawControlCurve(Input, Fixture.RawControlNames[1], 0.5f); // R.B = 0.5
	Op.EvaluateValues(*Fixture.Evaluator);
	const FPoseValueBundle& Output = TopBundle(*Fixture.Evaluator);

	const TMap<FName, float> Curves = SnapshotCurves(Output);
	const float* AB = Curves.Find(FName(TEXT("A_B")));
	if (AB == nullptr)
	{
		AddError(TEXT("A_B animated-map curve missing"));
		return false;
	}
	TestEqual(TEXT("A_B at R.B=0.5 hits conditional in active region (slope*0.5 + cut = 0.95)"), *AB, 0.95f, 1.e-4f);
	return true;
}

// =============================================================================
// Configuration gates - LoadJoints / LoadBlendShapes / LoadAnimatedMaps
//
// Mutate the UDNA's RigLogicConfiguration before CacheBones, then drive a fake
// PostEditChangeProperty event (only available WITH_EDITOR; this whole test
// suite already runs in EditorContext) so the runtime context rebuilds with
// the new config. Verify the corresponding Update*Curves / UpdateJoints branch
// is actually skipped.
// =============================================================================

namespace UE::UAF::Tests
{
	// Apply a RigLogicConfiguration override to the loaded UDNA and re-trigger
	// runtime context initialisation via PostEditChangeProperty.
	static void OverrideRigLogicConfig(UDNA* DNA, TFunctionRef<void(FRigLogicConfiguration&)> Mutator)
	{
		Mutator(DNA->RigLogicConfiguration);
		FProperty* Prop = FindFProperty<FProperty>(UDNA::StaticClass(), GET_MEMBER_NAME_CHECKED(UDNA, RigLogicConfiguration));
		FPropertyChangedEvent Event(Prop, EPropertyChangeType::ValueSet);
		DNA->PostEditChangeProperty(Event);
	}
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FRigLogicAnimOp_Config_LoadJointsFalseSkipsJoints, "RigLogic.RigLogicUAF.Config.LoadJointsFalseSkipsJoints", GTestFlags)
bool FRigLogicAnimOp_Config_LoadJointsFalseSkipsJoints::RunTest(const FString&)
{
	using namespace UE::UAF::Tests;
	using namespace UE::UAF;

	// Strategy: evaluate twice with identical input - once with the default config
	// (LoadJoints=true), once with LoadJoints=false - and assert that the variable
	// joints differ between the two runs while non-joint outputs (curves) are identical.
	//
	// We use a fresh FAnimOpFixture per evaluation because the AnimOp's InstanceData
	// pool caches per-mesh; reusing the same mesh across config changes risks stale
	// runtime context.

	auto EvaluateWith = [this](TFunctionRef<void(FRigLogicConfiguration&)> ConfigMutator)
		-> TTuple<bool, TArray<FTransform>, TMap<FName, float>>
	{
		FAnimOpFixture Fixture;
		if (!Fixture.Setup(*this, /*bUseLegacyDNA=*/false))
		{
			return {};
		}
		if (!Fixture.DNA.IsValid())
		{
			return {};
		}

		OverrideRigLogicConfig(Fixture.DNA.Get(), ConfigMutator);

		FRigLogicAnimOp Op;
		PushIdentityPose(*Fixture.Evaluator);
		FPoseValueBundle& Input = TopBundle(*Fixture.Evaluator);
		SeedAllRawControlsZero(Input, Fixture);
		SetRawControlCurve(Input, Fixture.RawControlNames[0], 1.0f);
		SetRawControlCurve(Input, Fixture.RawControlNames[3], 1.0f);
		SetRawControlCurve(Input, Fixture.RawControlNames[6], 1.0f);
		Op.EvaluateValues(*Fixture.Evaluator);
		const FPoseValueBundle& Output = TopBundle(*Fixture.Evaluator);

		TArray<FTransform> JointTransforms;
		// Snapshot the variable joints' transforms (in DNA-joint-index order).
		static const uint16 VariableJoints[] = { 0, 2, 4, 5, 6, 7 };
		for (const uint16 J : VariableJoints)
		{
			const FAttributeSetIndex SetIndex = Fixture.DNAJointToSetIndex[J];
			if (!SetIndex.IsValid())
			{
				JointTransforms.Add(FTransform::Identity);
				continue;
			}
			JointTransforms.Add((*Output.FindBoneTransforms())[SetIndex].Value);
		}
		return MakeTuple(true, JointTransforms, SnapshotCurves(Output));
	};

	auto Default = EvaluateWith([](FRigLogicConfiguration&) { /* leave defaults */ });
	auto NoJoints = EvaluateWith([](FRigLogicConfiguration& Cfg)
	{
		Cfg.LoadJoints = false;
	});
	if (!Default.Get<0>() || !NoJoints.Get<0>())
	{
		AddError(TEXT("Fixture setup failed"));
		return false;
	}

	const TArray<FTransform>& DefaultJoints = Default.Get<1>();
	const TArray<FTransform>& NoJointsJoints = NoJoints.Get<1>();

	// At least one variable joint must differ between the two configs.
	bool bAnyDiffer = false;
	for (int32 i = 0; i < DefaultJoints.Num(); ++i)
	{
		if (!DefaultJoints[i].Equals(NoJointsJoints[i], 1.e-3f))
		{
			bAnyDiffer = true;
			break;
		}
	}
	TestTrue(TEXT("Variable joint transforms must differ between LoadJoints=true and LoadJoints=false"), bAnyDiffer);

	// Every variable joint with LoadJoints=false must equal the input pose (identity).
	for (int32 i = 0; i < NoJointsJoints.Num(); ++i)
	{
		TestTrue(*FString::Printf(TEXT("Variable joint #%d should equal input pose with LoadJoints=false"), i), NoJointsJoints[i].Equals(FTransform::Identity, 1.e-3f));
	}

	// Curves must still be emitted (LoadBlendShapes / LoadAnimatedMaps still on).
	TestTrue(TEXT("Morph curves should still be emitted with LoadJoints=false"), NoJoints.Get<2>().Contains(FName(TEXT("MA__BA"))));
	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FRigLogicAnimOp_Config_LoadBlendShapesFalseSkipsMorphs, "RigLogic.RigLogicUAF.Config.LoadBlendShapesFalseSkipsMorphs", GTestFlags)
bool FRigLogicAnimOp_Config_LoadBlendShapesFalseSkipsMorphs::RunTest(const FString&)
{
	using namespace UE::UAF::Tests;
	using namespace UE::UAF;

	FAnimOpFixture Fixture;
	if (!Fixture.Setup(*this, /*bUseLegacyDNA=*/false))
	{
		return false;
	}

	if (!Fixture.DNA.IsValid())
	{
		AddError(TEXT("UDNA path required for this test"));
		return false;
	}

	OverrideRigLogicConfig(Fixture.DNA.Get(), [](FRigLogicConfiguration& Cfg)
	{
		Cfg.LoadBlendShapes = false;
	});

	FRigLogicAnimOp Op;
	PushIdentityPose(*Fixture.Evaluator);
	FPoseValueBundle& Input = TopBundle(*Fixture.Evaluator);
	SeedAllRawControlsZero(Input, Fixture);
	SetRawControlCurve(Input, Fixture.RawControlNames[0], 1.0f);
	Op.EvaluateValues(*Fixture.Evaluator);
	const FPoseValueBundle& Output = TopBundle(*Fixture.Evaluator);

	const TMap<FName, float> Curves = SnapshotCurves(Output);
	for (const FName& MorphName : Fixture.ExpectedMorphCurveNames)
	{
		const float* Value = Curves.Find(MorphName);
		// AnimOp leaves the curve at its input value (zero) when LoadBlendShapes is off.
		// We accept either "missing" or "zero" - both indicate LoadBlendShapes was honoured.
		TestFalse(*FString::Printf(TEXT("Morph curve %s should NOT be written with LoadBlendShapes=false"), *MorphName.ToString()), Value != nullptr && !FMath::IsNearlyZero(*Value, 1.e-4f));
	}
	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FRigLogicAnimOp_Config_LoadAnimatedMapsFalseSkipsAnimMaps, "RigLogic.RigLogicUAF.Config.LoadAnimatedMapsFalseSkipsAnimMaps", GTestFlags)
bool FRigLogicAnimOp_Config_LoadAnimatedMapsFalseSkipsAnimMaps::RunTest(const FString&)
{
	using namespace UE::UAF::Tests;
	using namespace UE::UAF;

	FAnimOpFixture Fixture;
	if (!Fixture.Setup(*this, /*bUseLegacyDNA=*/false))
	{
		return false;
	}

	if (!Fixture.DNA.IsValid())
	{
		AddError(TEXT("UDNA path required for this test"));
		return false;
	}

	OverrideRigLogicConfig(Fixture.DNA.Get(), [](FRigLogicConfiguration& Cfg)
	{
		Cfg.LoadAnimatedMaps = false;
	});

	FRigLogicAnimOp Op;
	PushIdentityPose(*Fixture.Evaluator);
	FPoseValueBundle& Input = TopBundle(*Fixture.Evaluator);
	SeedAllRawControlsZero(Input, Fixture);
	Op.EvaluateValues(*Fixture.Evaluator);
	const FPoseValueBundle& Output = TopBundle(*Fixture.Evaluator);

	const TMap<FName, float> Curves = SnapshotCurves(Output);
	for (const FName& AnimMapName : Fixture.ExpectedAnimMapCurveNames)
	{
		const float* Value = Curves.Find(AnimMapName);
		TestFalse(*FString::Printf(TEXT("Animated-map curve %s should NOT be written with LoadAnimatedMaps=false"), *AnimMapName.ToString()), Value != nullptr && !FMath::IsNearlyZero(*Value, 1.e-4f));
	}
	return true;
}

// =============================================================================
// Input curve clamping
//
// FAnimNode_RigLogic clamps incoming raw control curves to [0, 1] before forwarding
// them to FRigInstance::SetRawControl. FRigLogicAnimOp does NOT - it forwards values
// verbatim. This test pins that behaviour so a future change accidentally adding
// clamping would surface as a regression.
//
// We assert two things:
//   - R.B=2.0 produces values that pass through unclamped (e.g. MA__BB = 2.0, not 1.0)
//   - R.B=-0.5 produces values that pass through unclamped (e.g. MA__BB = -0.5, not 0)
// =============================================================================

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FRigLogicAnimOp_NoInputClamping, "RigLogic.RigLogicUAF.NoInputClamping", GTestFlags)
bool FRigLogicAnimOp_NoInputClamping::RunTest(const FString&)
{
	using namespace UE::UAF::Tests;
	using namespace UE::UAF;

	FAnimOpFixture Fixture;
	if (!Fixture.Setup(*this, /*bUseLegacyDNA=*/false))
	{
		return false;
	}

	// MA__BB is a direct passthrough of raw control 1, so it tracks the input value
	// even when out of [0, 1] range. This is the cleanest observable that proves no
	// clamping is happening.
	auto MaBb = [&](float ControlValue) -> float
	{
		FRigLogicAnimOp Op;
		PushIdentityPose(*Fixture.Evaluator);
		FPoseValueBundle& Input = TopBundle(*Fixture.Evaluator);
		SeedAllRawControlsZero(Input, Fixture);
		SetRawControlCurve(Input, Fixture.RawControlNames[1], ControlValue);
		Op.EvaluateValues(*Fixture.Evaluator);
		const TMap<FName, float> Curves = SnapshotCurves(TopBundle(*Fixture.Evaluator));
		const float* Ptr = Curves.Find(FName(TEXT("MA__BB")));
		return Ptr ? *Ptr : 0.0f;
	};

	TestEqual(TEXT("R.B=1.0 should pass through to MA__BB unchanged"), MaBb(1.0f), 1.0f, 1.e-4f);
	TestEqual(TEXT("R.B=2.0 should pass through to MA__BB without clamping"), MaBb(2.0f), 2.0f, 1.e-4f);
	TestEqual(TEXT("R.B=0 should produce zero on MA__BB"), MaBb(0.0f), 0.0f, 1.e-4f);
	TestEqual(TEXT("R.B=-0.5 should pass through to MA__BB without clamping"), MaBb(-0.5f), -0.5f, 1.e-4f);
	return true;
}

// =============================================================================
// LOD switching - TODO
//
// FRigLogicAnimOp picks the LOD via Evaluator.GetCurrentLOD(); the test fixture
// constructs the evaluator with LOD=0. A proper LOD test would parameterize the
// LOD, drive an evaluation at LOD 1, confirm that joints {4, 5, 7} are not
// written, and confirm that joint 0 still gets its full delta.
//
// Deferred: low priority since RigLogicLib already covers per-LOD behaviour at the
// engine level.
// =============================================================================

#endif // WITH_DEV_AUTOMATION_TESTS && WITH_EDITOR

#endif // RIGLOGIC_UAF_TESTS_ENABLED
