// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DNAIndexMapping.h"
#include "RigInstance.h"
#include "LODPose.h"
#include "Engine/SkeletalMesh.h"

struct FSharedRigRuntimeContext;
struct FDNAIndexMapping;

namespace UE::UAF
{
	struct FRigLogicBoneMapping
	{
		uint16 RigLogicJointIndex; /** RigLogic joint index based on the internal RigLogic skeleton. Bone indices differ from the skeleton or skeletal mesh bone indices and need translation. */
		int32 PoseBoneIndex; /** Pose bone index based on the given LOD level. */
	};

	struct FPoseBoneControlAttributeMapping
	{
		int32 PoseBoneIndex; /** Pose bone index based on the given LOD level. */
		int32 DNAJointIndex;
		int32 RotationX;
		int32 RotationY;
		int32 RotationZ;
		int32 RotationW;
	};

	// Instance data unique per reference pose, cloned by the maximum number of parallel evaluations. TODO: Better call it PoolData?
	struct FRigLogicInstanceData
	{
		/** Cached pointer to the shared RigLogic runtime context originally owned by UDNAAsset. */
		TSharedPtr<FSharedRigRuntimeContext> CachedRigRuntimeContext;

		/** Cached pointer to the DNA index mapping which is originally owned by UDNAAsset. */
		TSharedPtr<FDNAIndexMapping> CachedDNAIndexMapping;

		/** Bone index mapping from a RigLogic joint index to the reference skeleton bone index, one per LOD level. */
		TArray<TArray<FRigLogicBoneMapping>> RigLogicToSkeletonBoneIndexMappingPerLOD;

		TArray<TArray<FPoseBoneControlAttributeMapping>> SparseDriverJointsToControlAttributesMapPerLOD;
		TArray<TArray<FPoseBoneControlAttributeMapping>> DenseDriverJointsToControlAttributesMapPerLOD;

		uint32 NumLODs = 0;

		void Init(const UE::UAF::FReferencePose* ReferencePose);

	private:
		void InitBoneIndexMapping(const UE::UAF::FReferencePose* ReferencePose);
		void InitSparseAndDenseDriverJointMapping(const UE::UAF::FReferencePose* ReferencePose);
	};
} // namespace UE::UAF