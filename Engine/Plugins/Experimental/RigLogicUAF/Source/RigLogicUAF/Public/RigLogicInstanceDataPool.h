// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RigLogicInstanceData.h"

namespace UE::UAF
{
	class FRigLogicInstanceDataPool
	{
	public:
		TMap<TWeakObjectPtr<const USkeletalMesh>, TArray<TSharedPtr<FRigLogicInstanceData>>> Datas;

		mutable FCriticalSection PoolAccessCriticalSection;

		TSharedPtr<FRigLogicInstanceData> RequestData(const UE::UAF::FReferencePose* InReferencePose);
		void FreeData(TWeakObjectPtr<const USkeletalMesh> SkeletalMesh, TSharedPtr<FRigLogicInstanceData> InData);

		void GarbageCollect();
		void Log();
	};
} // namespace UE::UAF