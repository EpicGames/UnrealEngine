// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "EvaluationVM/EvaluationTask.h"
#include "Animation/AnimCurveTypes.h"
#include "LODPose.h"
#include "AnimNextStats.h"

#include "RigInstance.h"
#include "RigLogic.h"
#include "RigLogicTrait.h"
#include "RigLogicInstanceData.h"

#include "RigLogicTask.generated.h"

struct FSharedRigRuntimeContext;
struct FDNAIndexMapping;

DECLARE_CYCLE_STAT_EXTERN(TEXT("UAF Task: RigLogic"), STAT_UAF_Task_RigLogic, STATGROUP_AnimNext, RIGLOGICUAF_API);

USTRUCT()
struct RIGLOGICUAF_API FUAFRigLogicTask : public FAnimNextEvaluationTask
{
	GENERATED_BODY()

	DECLARE_ANIM_EVALUATION_TASK(FUAFRigLogicTask)

	static FUAFRigLogicTask Make(UE::UAF::FRigLogicTrait::FInstanceData* InstanceData);

	// Task entry point
	virtual void Execute(UE::UAF::FEvaluationVM& VM) const override;

	using FCachedIndexedCurve = TBaseBlendedCurve<FDefaultAllocator, UE::Anim::FCurveElementIndexed>;
	static constexpr uint16 ATTR_COUNT_PER_JOINT = 10;

	void UpdateControlCurves(FRigInstance* InRigInstance,
		const FBlendedCurve& Curves,
		const FCachedIndexedCurve& ControlAttributeCurves,
		const FCachedIndexedCurve& MLMaskCurves,
		const TArrayView<const float> NeutralJointValues,
		const TArrayView<const UE::UAF::FPoseBoneControlAttributeMapping> SparseDriverJointsToControlAttributesMap,
		const TArrayView<const UE::UAF::FPoseBoneControlAttributeMapping> DenseDriverJointsToControlAttributesMap,
		const UE::UAF::FLODPoseStack& InputPose) const;

	static void UpdateJoints(const TArrayView<const UE::UAF::FRigLogicBoneMapping> RigLogicIndexToMeshIndexMapping,
		const TArrayView<const uint16> VariableJointIndices,
		const TArrayView<const float> NeutralJointValues,
		const TArrayView<const float> DeltaJointValues,
		UE::UAF::FLODPoseStack& OutputPose);

	static void UpdateBlendShapeCurves(const FCachedIndexedCurve& MorphTargetCurves, const TArrayView<const float> BlendShapeValues, FBlendedCurve& OutputCurves);
	static void UpdateAnimMapCurves(const FCachedIndexedCurve& MaskMultiplierCurves, const TArrayView<const float> AnimMapOutputs, FBlendedCurve& OutputCurves);

	UE::UAF::FRigLogicTrait::FInstanceData* TraitInstanceData = nullptr;
};