// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "RigInstance.h"
#include "TraitCore/Trait.h"
#include "TraitInterfaces/IContinuousBlend.h"
#include "TraitInterfaces/IEvaluate.h"
#include "TraitInterfaces/IHierarchy.h"
#include "TraitInterfaces/IUpdate.h"

#include "RigLogicTrait.generated.h"

USTRUCT(meta = (DisplayName = "RigLogic"))
struct RIGLOGICUAF_API FUAFRigLogicTraitSharedData : public FAnimNextTraitSharedData
{
	GENERATED_BODY()

	UPROPERTY()
	FAnimNextTraitHandle Input;

	/*
	 * Max LOD level that post-process AnimBPs are evaluated.
	 * For example if you have the threshold set to 2, it will evaluate until including LOD 2 (based on 0 index). In case the LOD level gets set to 3, it will stop evaluating the post-process AnimBP.
	 * Setting it to -1 will always evaluate it and disable LODing.
	 */
	UPROPERTY(EditAnywhere, Category = RigLogic)
	int32 LODThreshold = INDEX_NONE;

	// Latent pin support boilerplate
#define TRAIT_LATENT_PROPERTIES_ENUMERATOR(GeneratorMacro) \
		GeneratorMacro(LODThreshold)

	GENERATE_TRAIT_LATENT_PROPERTIES(FUAFRigLogicTraitSharedData, TRAIT_LATENT_PROPERTIES_ENUMERATOR)
#undef TRAIT_LATENT_PROPERTIES_ENUMERATOR
};

namespace UE::UAF
{
	/**
	 * FRigLogicTrait
	 * 
	 * A trait that can run RigLogic.
	 */
	struct RIGLOGICUAF_API FRigLogicTrait : FBaseTrait, IEvaluate, IUpdate, IUpdateTraversal, IHierarchy
	{
		DECLARE_ANIM_TRAIT(FRigLogicTrait, FBaseTrait)

		using FSharedData = FUAFRigLogicTraitSharedData;

		// Do we need this? Or is the input in the shared data sufficient?
		struct FInstanceData : FTrait::FInstanceData
		{
			/** Input node from which we receive the input pose as well as the facial expression curves. */
			FTraitPtr Input;

			/** Actually cloned RigLogic instance owned by this class. */
			TUniquePtr<FRigInstance> RigInstance;
		};

		// IEvaluate impl
		virtual void PostEvaluate(FEvaluateTraversalContext& Context, const TTraitBinding<IEvaluate>& Binding) const override;

		// IUpdate impl
		// Called before the first update when a trait stack becomes relevant
		virtual void OnBecomeRelevant(FUpdateTraversalContext& Context, const TTraitBinding<IUpdate>& Binding, const FTraitUpdateState& TraitState) const override;

		// IUpdateTraversal impl
		virtual void QueueChildrenForTraversal(FUpdateTraversalContext& Context, const TTraitBinding<IUpdateTraversal>& Binding, const FTraitUpdateState& TraitState, FUpdateTraversalQueue& TraversalQueue) const override;

		// IHierarchy impl
		virtual uint32 GetNumChildren(const FExecutionContext& Context, const TTraitBinding<IHierarchy>& Binding) const override;
		virtual void GetChildren(const FExecutionContext& Context, const TTraitBinding<IHierarchy>& Binding, FChildrenArray& Children) const override;
	};
} // namespace UE::UAF