// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleManager.h"
#include "RigLogicInstanceDataPool.h"

DECLARE_LOG_CATEGORY_EXTERN(LogRigLogicUAF, Log, All);

namespace UE::UAF
{
	class FRigLogicModule : public IModuleInterface
	{
	public:
		/** IModuleInterface implementation */
		virtual void StartupModule() override;
		virtual void ShutdownModule() override;

		UE::UAF::FRigLogicInstanceDataPool DataPool;
	};
} // namespace UE::UAF