// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class RigLogicUAF : ModuleRules
{
	public RigLogicUAF(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;
		
		PublicIncludePaths.AddRange(
			new string[]
			{
			}
			);
				
		
		PrivateIncludePaths.AddRange(
			new string[]
			{
			}
			);
			
		
		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"Engine",
				"RigLogicModule",
				"UAF",
				"UAFAnimGraph"
			}
			);
			
		
		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"CoreUObject",
				"Engine",
				"Slate",
				"SlateCore",
			}
			);
		
		
		DynamicallyLoadedModuleNames.AddRange(
			new string[]
			{
			}
			);
	}
}
