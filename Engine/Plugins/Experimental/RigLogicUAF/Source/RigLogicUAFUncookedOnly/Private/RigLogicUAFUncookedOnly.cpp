// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigLogicUAFUncookedOnly.h"
#include "Engine/Texture2D.h"
#include "Styling/SlateStyle.h"
#include "Styling/SlateStyleRegistry.h"
#include "Brushes/SlateImageBrush.h"
#include "Interfaces/IPluginManager.h"

DEFINE_LOG_CATEGORY(LogRigLogicUAFUncookedOnly);

namespace UE::UAF
{
    void FRigLogicModuleUncookedOnly::StartupModule()
    {
    }

    void FRigLogicModuleUncookedOnly::ShutdownModule()
    {
    }

	const FSlateBrush& FRigLogicModuleUncookedOnly::GetIcon()
	{
    	static TSharedPtr<FSlateStyleSet> StyleSet = nullptr;
    	static const FSlateBrush* RegisteredBrush = nullptr;

    	if (!StyleSet.IsValid())
    	{
    		StyleSet = MakeShareable(new FSlateStyleSet("RigLogicStyleUAF"));

    		const FString PluginPath = IPluginManager::Get().FindPlugin("RigLogicUAF")->GetBaseDir();
    		StyleSet->SetContentRoot(PluginPath / TEXT("Resources"));

    		const FName BrushName = "RigLogicUAF_Icon";
    		StyleSet->Set(BrushName, new FSlateImageBrush(StyleSet->RootToContentDir(TEXT("Icon128"), TEXT(".png")), FVector2D(16, 16)));

    		FSlateStyleRegistry::RegisterSlateStyle(*StyleSet);
    		RegisteredBrush = StyleSet->GetBrush(BrushName);
    	}

    	return *RegisteredBrush;
	}
} // namespace UE::UAF

IMPLEMENT_MODULE(UE::UAF::FRigLogicModuleUncookedOnly, RigLogicUAFUncookedOnly)