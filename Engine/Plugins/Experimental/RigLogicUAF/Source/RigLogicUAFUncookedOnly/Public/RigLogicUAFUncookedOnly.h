// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleManager.h"
#include "Textures/SlateIcon.h"

DECLARE_LOG_CATEGORY_EXTERN(LogRigLogicUAFUncookedOnly, Log, All);

namespace UE::UAF
{
	class FRigLogicModuleUncookedOnly : public IModuleInterface
	{
	public:
		/** IModuleInterface implementation */
		virtual void StartupModule() override;
		virtual void ShutdownModule() override;

		static const FSlateBrush& GetIcon();
	};
} // namespace UE::UAF