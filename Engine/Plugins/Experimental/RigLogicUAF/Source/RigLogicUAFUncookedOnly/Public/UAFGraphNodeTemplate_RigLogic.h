// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimGraphUncookedOnlyUtils.h"
#include "Templates/UAFGraphNodeTemplate.h"
#include "RigLogicTrait.h"
#include "Traits/PassthroughBlendTrait.h"
#include "RigLogicUAFUncookedOnly.h"

#include "UAFGraphNodeTemplate_RigLogic.generated.h"

#define LOCTEXT_NAMESPACE "UAFGraphNodeTemplate_RigLogic"

UCLASS()
class UUAFGraphNodeTemplate_RigLogic : public UUAFGraphNodeTemplate
{
	GENERATED_BODY()

	UUAFGraphNodeTemplate_RigLogic()
	{
		Title = LOCTEXT("RigLogicUAFTitle", "RigLogic");
		TooltipText = LOCTEXT("RigLogicUAFTooltip", "RigLogic\n"
			"Performs facial animation and drives body correctives where applicable.\n"
			"Input: Input Pose\n"
			"Output: Overwritten joint transforms, and curves to drive blend shapes and animated maps.");
		Category = LOCTEXT("RigLogicUAFCategory", "UAF");
		MenuDescription = LOCTEXT("RigLogicUAFMenuDesc", "RigLogic");
		Color = FLinearColor(FColor(38, 187, 255));
		Icon = UE::UAF::FRigLogicModuleUncookedOnly::GetIcon();

		FPassthroughBlendTraitSharedData PassthroughData;
		PassthroughData.AlphaInputType = EAnimAlphaInputType::Bool;

		Traits =
		{
			TInstancedStruct<FUAFRigLogicTraitSharedData>::Make(),
			TInstancedStruct<FPassthroughBlendTraitSharedData>::Make(PassthroughData)
		};
		SetCategoryForPinsInLayout(
			{
				GET_PIN_PATH_STRING_CHECKED(FUAFRigLogicTraitSharedData, Input),
				GET_PIN_PATH_STRING_CHECKED(FPassthroughBlendTraitSharedData, bAlphaBoolEnabled)
			},
			TEXT("Default"),
			NodeLayout,
			true);
		SetDisplayNameForPinInLayout(GET_PIN_PATH_STRING_CHECKED(FUAFRigLogicTraitSharedData, Input), TEXT("Input"), NodeLayout);
		SetDisplayNameForPinInLayout(GET_PIN_PATH_STRING_CHECKED(FPassthroughBlendTraitSharedData, bAlphaBoolEnabled), TEXT("Enabled"), NodeLayout);
	}
};

#undef LOCTEXT_NAMESPACE